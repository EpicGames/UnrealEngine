{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "PlayerInputDebugger",
	"Description": "Plugin for debugging input related things",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"EditorOnly": true,
	"Modules": [
		{
			"Name": "PlayerInputDebugger",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		}
	],
	"Plugins": [
		{
			"Name": "EnhancedInput",
			"Enabled": true
		},
		{
			"Name": "CommonUI",
			"Enabled": true
		}
	]
}
