// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayerInputDebuggerModule.h"

#include "SPlayerInputDebugger.h"

#include "Framework/Application/SlateApplication.h"
#include "Framework/Docking/TabManager.h"
#include "Modules/ModuleManager.h"
#include "ToolMenus.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "FPlayerInputDebuggerModule"

static const FName PlayerInputDebuggerTabName("PlayerInputDebugger");

void FPlayerInputDebuggerModule::StartupModule()
{
	MenusStartupHandle = UToolMenus::RegisterStartupCallback(
		FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FPlayerInputDebuggerModule::RegisterMenus));
}

void FPlayerInputDebuggerModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(MenusStartupHandle);
	UToolMenus::UnRegisterStartupCallback(this);

	if (UToolMenus* ToolMenus = UToolMenus::TryGet())
	{
		ToolMenus->UnregisterOwnerByName("PlayerInputDebugger");
	}

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(PlayerInputDebuggerTabName);
}

void FPlayerInputDebuggerModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped("PlayerInputDebugger");

	const IWorkspaceMenuStructure& MenuStructure = WorkspaceMenu::GetMenuStructure();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		PlayerInputDebuggerTabName,
		FOnSpawnTab::CreateRaw(this, &FPlayerInputDebuggerModule::SpawnTab))
		.SetDisplayName(LOCTEXT("TabTitle", "Player Input Debugger"))
		.SetTooltipText(LOCTEXT("TabTooltip", "Open the Player Input Debugger window to inspect input events, connected devices, and Enhanced Input state."))
		.SetGroup(MenuStructure.GetDeveloperToolsDebugCategory())
		.SetIcon(FSlateIcon("EnhancedInputEditor", "EnhancedInputIcon_Small"));
}

TSharedRef<SDockTab> FPlayerInputDebuggerModule::SpawnTab(const FSpawnTabArgs& Args)
{
	// The nomad tab must exist before SPlayerInputDebugger is constructed because
	// it needs to pass itself to FGlobalTabmanager::NewTabManager to create the
	// local tab manager that drives the dockable sub-tabs.
	TSharedRef<SDockTab> NomadTab = SNew(SDockTab)
		.TabRole(ETabRole::NomadTab);

	NomadTab->SetContent(
		SNew(SPlayerInputDebugger)
		.ParentTab(NomadTab)
	);

	return NomadTab;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPlayerInputDebuggerModule, PlayerInputDebugger)
