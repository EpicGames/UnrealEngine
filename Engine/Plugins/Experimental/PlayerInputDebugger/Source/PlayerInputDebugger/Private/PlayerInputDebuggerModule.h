// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class FSpawnTabArgs;
class SDockTab;

class FPlayerInputDebuggerModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	void RegisterMenus();
	TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs& Args);

	FDelegateHandle MenusStartupHandle;
};
