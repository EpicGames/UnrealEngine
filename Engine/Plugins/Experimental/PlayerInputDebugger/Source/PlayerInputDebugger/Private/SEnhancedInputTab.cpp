// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEnhancedInputTab.h"

#include "EnhancedActionKeyMapping.h"
#include "EnhancedInputSubsystems.h"
#include "GameplayTagContainer.h"
#include "EnhancedPlayerInput.h"
#include "InputAction.h"
#include "InputMappingContext.h"
#include "UObject/UObjectIterator.h"
#include "Editor.h"
#include "Engine/LocalPlayer.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"
#include "UserSettings/EnhancedInputUserSettings.h"
#include "PropertyCustomizationHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SEnhancedInputTab"

namespace EnhancedInputTab
{
	static const float PriorityColumnWidth = 65.f;
	static const float ContextColumnWidth  = 200.f;
	static const float ActionColumnWidth   = 200.f;
	static const float KeyColumnWidth      = 160.f;
	static const float StateColumnWidth    = 90.f;
	static const float ValueColumnWidth    = 150.f;
}

SEnhancedInputTab::~SEnhancedInputTab()
{
}

void SEnhancedInputTab::SetPlayerController(APlayerController* PC)
{
	WeakPC = PC;
}

// ETriggerEvent is a flags enum: None=0, Triggered=1, Started=2, Ongoing=4, Canceled=8, Completed=16.
// FEnhancedInputDisplayItem::TriggerEventOrdinal stores the raw flag value, not a 0..5 index.
static FSlateColor GetTriggerEventColor(uint8 Ordinal)
{
	switch (static_cast<ETriggerEvent>(Ordinal))
	{
		case ETriggerEvent::Triggered: return FLinearColor(0.2f, 0.9f, 0.2f);  // green
		case ETriggerEvent::Started:   return FLinearColor(0.2f, 0.8f, 0.9f);  // cyan
		case ETriggerEvent::Ongoing:   return FLinearColor(0.9f, 0.9f, 0.2f);  // yellow
		case ETriggerEvent::Canceled:  return FLinearColor(0.9f, 0.2f, 0.2f);  // red
		case ETriggerEvent::Completed: return FLinearColor(0.5f, 0.5f, 0.5f);  // gray
		default: return FLinearColor(0.8f, 0.8f, 0.8f);                        // None - dim white
	}
}

static FText GetTriggerEventName(uint8 Ordinal)
{
	switch (static_cast<ETriggerEvent>(Ordinal))
	{
		case ETriggerEvent::Triggered: return LOCTEXT("TETriggered", "Triggered");
		case ETriggerEvent::Started:   return LOCTEXT("TEStarted",   "Started");
		case ETriggerEvent::Ongoing:   return LOCTEXT("TEOngoing",   "Ongoing");
		case ETriggerEvent::Canceled:  return LOCTEXT("TECanceled",  "Canceled");
		case ETriggerEvent::Completed: return LOCTEXT("TECompleted", "Completed");
		case ETriggerEvent::None:      return LOCTEXT("TENone",      "None");
		default: return FText::AsNumber(Ordinal);
	}
}

static const FHyperlinkStyle& GetYellowHyperlinkStyle()
{
	static const FHyperlinkStyle Style = []()
	{
		FHyperlinkStyle S = FAppStyle::Get().GetWidgetStyle<FHyperlinkStyle>("Hyperlink");
		S.SetTextStyle(FTextBlockStyle(S.TextStyle)
			.SetColorAndOpacity(FSlateColor(FLinearColor(0.9f, 0.8f, 0.2f))));
		return S;
	}();
	return Style;
}

static void OpenAssetEditor(TWeakObjectPtr<const UObject> WeakAsset)
{
	const UObject* Asset = WeakAsset.Get();
	if (!Asset || !GEditor) { return; }

	const FString PathName = Asset->GetPathName();

	UObject* MutableAsset = FindObject<UObject>(nullptr, *PathName);
	if (!MutableAsset)
	{
		MutableAsset = LoadObject<UObject>(nullptr, *PathName);
	}
	if (!MutableAsset) { return; }

	if (UAssetEditorSubsystem* Sub = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
	{
		Sub->OpenEditorForAsset(MutableAsset);
	}
}

class SEnhancedInputContextHeaderRow : public SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>
{
public:
	SLATE_BEGIN_ARGS(SEnhancedInputContextHeaderRow) {}
		SLATE_ARGUMENT(TSharedPtr<FEnhancedInputDisplayItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>::Construct(
			FSuperRowType::FArguments().Padding(FMargin(2.f, 2.f)),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(Item.IsValid());

		if (ColumnName == TEXT("Priority"))
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(Item->Priority))
				.Font(FAppStyle::GetFontStyle("BoldFont"))
				.ColorAndOpacity(FLinearColor(0.9f, 0.8f, 0.2f));
		}

		if (ColumnName == TEXT("Context"))
		{
			if (Item->WeakAsset.IsValid())
			{
				TWeakObjectPtr<const UObject> WeakIMC = Item->WeakAsset;
				return SNew(SHyperlink)
					.Text(Item->ContextName)
					.Style(&GetYellowHyperlinkStyle())
					.ToolTipText(LOCTEXT("OpenIMCTooltip", "Click to open this Input Mapping Context in the asset editor"))
					.OnNavigate(FSimpleDelegate::CreateLambda([WeakIMC]()
					{
						OpenAssetEditor(WeakIMC);
					}));
			}
			return SNew(STextBlock)
				.Text(Item->ContextName)
				.Font(FAppStyle::GetFontStyle("BoldFont"))
				.ColorAndOpacity(FLinearColor(0.9f, 0.8f, 0.2f));
		}

		return SNew(STextBlock).Text(FText::GetEmpty());
	}

private:
	TSharedPtr<FEnhancedInputDisplayItem> Item;
};

class SEnhancedInputRow : public SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>
{
public:
	SLATE_BEGIN_ARGS(SEnhancedInputRow) {}
		SLATE_ARGUMENT(TSharedPtr<FEnhancedInputDisplayItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>::Construct(
			FSuperRowType::FArguments().Padding(FMargin(2.f, 1.f)),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(Item.IsValid());

		// Priority and Context columns are blank for action rows
		if (ColumnName == TEXT("Priority") || ColumnName == TEXT("Context"))
		{
			return SNew(STextBlock).Text(FText::GetEmpty());
		}

		if (ColumnName == TEXT("Action"))
		{
			if (Item->WeakAsset.IsValid())
			{
				TWeakObjectPtr<const UObject> WeakAction = Item->WeakAsset;
				return SNew(SHyperlink)
					.Text(Item->ActionName)
					.ToolTipText(LOCTEXT("OpenActionTooltip", "Click to open this Input Action in the asset editor"))
					.OnNavigate(FSimpleDelegate::CreateLambda([WeakAction]()
					{
						OpenAssetEditor(WeakAction);
					}));
			}
			return SNew(STextBlock)
				.Text(Item->ActionName)
				.Font(FAppStyle::GetFontStyle("NormalFont"));
		}

		if (ColumnName == TEXT("Key"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->BoundKey.ToString()))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f));
		}

		if (ColumnName == TEXT("State"))
		{
			TWeakPtr<FEnhancedInputDisplayItem> WeakItem = Item;
			return SNew(STextBlock)
				.Text_Lambda([WeakItem]() -> FText
				{
					if (TSharedPtr<FEnhancedInputDisplayItem> Pinned = WeakItem.Pin())
					{
						return GetTriggerEventName(Pinned->TriggerEventOrdinal);
					}
					return FText::GetEmpty();
				})
				.ColorAndOpacity_Lambda([WeakItem]() -> FSlateColor
				{
					if (TSharedPtr<FEnhancedInputDisplayItem> Pinned = WeakItem.Pin())
					{
						return GetTriggerEventColor(Pinned->TriggerEventOrdinal);
					}
					return FSlateColor(FLinearColor(0.8f, 0.8f, 0.8f));
				})
				.Font(FAppStyle::GetFontStyle("NormalFont"));
		}

		if (ColumnName == TEXT("Value"))
		{
			TWeakPtr<FEnhancedInputDisplayItem> WeakItem = Item;
			return SNew(STextBlock)
				.Text_Lambda([WeakItem]() -> FText
				{
					if (TSharedPtr<FEnhancedInputDisplayItem> Pinned = WeakItem.Pin())
					{
						return Pinned->ValueText;
					}
					return FText::GetEmpty();
				})
				.Font(FAppStyle::GetFontStyle("NormalFont"));
		}

		return SNew(STextBlock).Text(FText::GetEmpty());
	}

private:
	TSharedPtr<FEnhancedInputDisplayItem> Item;
};

// Sub-row displayed directly below each IMC header showing input mode filter settings and registration tracking.
class SEnhancedInputModeInfoRow : public SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>
{
public:
	SLATE_BEGIN_ARGS(SEnhancedInputModeInfoRow) {}
		SLATE_ARGUMENT(TSharedPtr<FEnhancedInputDisplayItem>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FEnhancedInputDisplayItem>>::Construct(
			FSuperRowType::FArguments().Padding(FMargin(2.f, 1.f)),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(Item.IsValid());

		static const FLinearColor LabelColor(0.5f, 0.5f, 0.5f);
		static const FLinearColor ValueColor(0.8f, 0.8f, 0.8f);
		static const FLinearColor PassesColor(0.2f, 0.85f, 0.2f);
		static const FLinearColor FilteredColor(0.9f, 0.55f, 0.1f);

		const EMappingContextInputModeFilterOptions FilterOptions =
			static_cast<EMappingContextInputModeFilterOptions>(Item->FilterOptionsOrdinal);
		const EMappingContextRegistrationTrackingMode RegMode =
			static_cast<EMappingContextRegistrationTrackingMode>(Item->RegTrackingModeOrdinal);

		// Helper: small dimmed label + normal value side by side
		auto MakeLabelValue = [&](FText Label, FText Value, FLinearColor ValueCol) -> TSharedRef<SWidget>
		{
			return SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center).Padding(0.f, 0.f, 4.f, 0.f)
				[
					SNew(STextBlock)
					.Text(Label)
					.Font(FAppStyle::GetFontStyle("SmallFont"))
					.ColorAndOpacity(LabelColor)
				]
				+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Value)
					.Font(FAppStyle::GetFontStyle("SmallFont"))
					.ColorAndOpacity(ValueCol)
				];
		};

		if (ColumnName == TEXT("Priority"))
		{
			return SNew(STextBlock).Text(FText::GetEmpty());
		}

		if (ColumnName == TEXT("Context"))
		{
			FText FilterModeText;
			switch (FilterOptions)
			{
			case EMappingContextInputModeFilterOptions::UseCustomQuery:
				FilterModeText = LOCTEXT("FilterCustom",  "Custom Query");   break;
			case EMappingContextInputModeFilterOptions::DoNotFilter:
				FilterModeText = LOCTEXT("FilterDisabled","Don't Filter");   break;
			default:
				FilterModeText = LOCTEXT("FilterDefault", "Project Default"); break;
			}
			return MakeLabelValue(LOCTEXT("FilterLabel", "Filter:"), FilterModeText, ValueColor);
		}

		if (ColumnName == TEXT("Action"))
		{
			return MakeLabelValue(LOCTEXT("QueryLabel", "Query:"), Item->ModeQueryText, ValueColor);
		}

		if (ColumnName == TEXT("Key"))
		{
			return SNew(STextBlock).Text(FText::GetEmpty());
		}

		if (ColumnName == TEXT("State"))
		{
			// Show whether this IMC passes the current EI subsystem input mode filter.
			// Uses TAttribute lambdas so the widget survives per-frame structural comparisons.
			TWeakPtr<FEnhancedInputDisplayItem> WeakItem = Item;
			return SNew(STextBlock)
				.Text_Lambda([WeakItem]() -> FText
				{
					if (TSharedPtr<FEnhancedInputDisplayItem> Pinned = WeakItem.Pin())
					{
						const EMappingContextInputModeFilterOptions F =
							static_cast<EMappingContextInputModeFilterOptions>(Pinned->FilterOptionsOrdinal);
						if (F == EMappingContextInputModeFilterOptions::DoNotFilter || !Pinned->bShouldApplyModeFilter)
						{
							return LOCTEXT("ModeNA", "—");
						}
						return Pinned->bPassesModeFilter
							? LOCTEXT("ModePasses",   "Passes")
							: LOCTEXT("ModeFiltered", "Filtered");
					}
					return FText::GetEmpty();
				})
				.ColorAndOpacity_Lambda([WeakItem]() -> FSlateColor
				{
					if (TSharedPtr<FEnhancedInputDisplayItem> Pinned = WeakItem.Pin())
					{
						const EMappingContextInputModeFilterOptions F =
							static_cast<EMappingContextInputModeFilterOptions>(Pinned->FilterOptionsOrdinal);
						if (F == EMappingContextInputModeFilterOptions::DoNotFilter || !Pinned->bShouldApplyModeFilter)
						{
							return FSlateColor(LabelColor);
						}
						return FSlateColor(Pinned->bPassesModeFilter ? PassesColor : FilteredColor);
					}
					return FSlateColor(LabelColor);
				})
				.Font(FAppStyle::GetFontStyle("SmallFont"))
				.ToolTipText(LOCTEXT("ModeFilterTooltip", "Whether this IMC passes the current Enhanced Input subsystem input mode filter"));
		}

		if (ColumnName == TEXT("Value"))
		{
			FText RegText = (RegMode == EMappingContextRegistrationTrackingMode::CountRegistrations)
				? LOCTEXT("RegCount",     "Count Regs")
				: LOCTEXT("RegUntracked", "Untracked");
			return MakeLabelValue(LOCTEXT("RegLabel", "Reg:"), RegText, ValueColor);
		}

		return SNew(STextBlock).Text(FText::GetEmpty());
	}

private:
	TSharedPtr<FEnhancedInputDisplayItem> Item;
};

void SEnhancedInputTab::Construct(const FArguments& InArgs)
{
	SetPlayerController(InArgs._PlayerController.Get());

	SetCanTick(true);

	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column("Priority")
		.DefaultLabel(LOCTEXT("ColPriority", "Priority"))
		.ManualWidth(EnhancedInputTab::PriorityColumnWidth)

		+ SHeaderRow::Column("Context")
		.DefaultLabel(LOCTEXT("ColContext", "Context"))
		.ManualWidth(EnhancedInputTab::ContextColumnWidth)

		+ SHeaderRow::Column("Action")
		.DefaultLabel(LOCTEXT("ColAction", "Action"))
		.ManualWidth(EnhancedInputTab::ActionColumnWidth)

		+ SHeaderRow::Column("Key")
		.DefaultLabel(LOCTEXT("ColKey", "Key"))
		.ManualWidth(EnhancedInputTab::KeyColumnWidth)
		.OnGetMenuContent(FOnGetContent::CreateSP(this, &SEnhancedInputTab::MakeKeyFilterMenuContent))

		+ SHeaderRow::Column("State")
		.DefaultLabel(LOCTEXT("ColState", "Trigger State"))
		.ManualWidth(EnhancedInputTab::StateColumnWidth)

		+ SHeaderRow::Column("Value")
		.DefaultLabel(LOCTEXT("ColValue", "Value"))
		.ManualWidth(EnhancedInputTab::ValueColumnWidth);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.f, 8.f, 8.f, 4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Header", "Active Mapping Contexts (highest priority first)"))
			.Font(FAppStyle::GetFontStyle("HeadingExtraSmall"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.AreaTitle(LOCTEXT("ContextFiltersTitle", "Context Filters"))
			.InitiallyCollapsed(false)
			.BodyContent()
			[
				SAssignNew(FilterTagsContainer, SBox)
				.Padding(FMargin(4.f, 4.f))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.AreaTitle(LOCTEXT("ActionFiltersTitle", "Action Filters"))
			.InitiallyCollapsed(false)
			.BodyContent()
			[
				SAssignNew(ActionFilterTagsContainer, SBox)
				.Padding(FMargin(4.f, 4.f))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SExpandableArea)
			.AreaTitle(LOCTEXT("KeyProfilesTitle", "Mappable Key Profiles"))
			.InitiallyCollapsed(false)
			.BodyContent()
			[
				SNew(SVerticalBox)

				// Active profile display
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f, 4.f, 4.f, 2.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 6.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ActiveProfileLabel", "Active Profile:"))
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text_Lambda([this]() -> FText
						{
							APlayerController* PC = WeakPC.Get();
							if (!PC) { return LOCTEXT("NoPC", "(no player controller)"); }
							ULocalPlayer* LP = PC->GetLocalPlayer();
							if (!LP) { return LOCTEXT("NoLP", "(no local player)"); }
							UEnhancedInputLocalPlayerSubsystem* EIS = LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>();
							if (!EIS) { return LOCTEXT("NoEIS", "(no EIS)"); }
							UEnhancedInputUserSettings* Settings = EIS->GetUserSettings();
							if (!Settings) { return LOCTEXT("NoSettings", "(no user settings)"); }
							return FText::FromString(Settings->GetActiveKeyProfileId());
						})
						.ColorAndOpacity(FLinearColor(0.2f, 0.9f, 0.2f))
					]
				]

				// Profile ID text entry
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f, 2.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(0.f, 0.f, 6.f, 0.f)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ProfileIdLabel", "Profile ID:"))
					]

					+ SHorizontalBox::Slot()
					.FillWidth(1.f)
					[
						SAssignNew(ProfileIdInput, SEditableTextBox)
						.HintText(LOCTEXT("ProfileIdHint", "Enter profile identifier..."))
					]
				]

				// Action buttons
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.f, 2.f, 4.f, 4.f)
				[
					SNew(SHorizontalBox)

					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(0.f, 0.f, 6.f, 0.f)
					[
						SNew(SButton)
						.Text(LOCTEXT("SetProfileBtn", "Set Mapping Profile"))
						.OnClicked_Lambda([this]() -> FReply
						{
							if (!ProfileIdInput.IsValid()) { return FReply::Handled(); }
							const FString ProfileId = ProfileIdInput->GetText().ToString();
							if (ProfileId.IsEmpty()) { return FReply::Handled(); }
							APlayerController* PC = WeakPC.Get();
							if (!PC) { return FReply::Handled(); }
							ULocalPlayer* LP = PC->GetLocalPlayer();
							if (!LP) { return FReply::Handled(); }
							UEnhancedInputLocalPlayerSubsystem* EIS = LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>();
							if (!EIS) { return FReply::Handled(); }
							UEnhancedInputUserSettings* Settings = EIS->GetUserSettings();
							if (!Settings) { return FReply::Handled(); }
							Settings->SetActiveKeyProfile(ProfileId);
							return FReply::Handled();
						})
					]

					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.Text(LOCTEXT("ResetProfileBtn", "Reset to Default"))
						.OnClicked_Lambda([this]() -> FReply
						{
							APlayerController* PC = WeakPC.Get();
							if (!PC) { return FReply::Handled(); }
							ULocalPlayer* LP = PC->GetLocalPlayer();
							if (!LP) { return FReply::Handled(); }
							UEnhancedInputLocalPlayerSubsystem* EIS = LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>();
							if (!EIS) { return FReply::Handled(); }
							UEnhancedInputUserSettings* Settings = EIS->GetUserSettings();
							if (!Settings) { return FReply::Handled(); }
							Settings->SetKeyProfileToDefault();
							return FReply::Handled();
						})
					]
				]
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SAssignNew(ListView, SListView<TSharedPtr<FEnhancedInputDisplayItem>>)
				.ListItemsSource(&DisplayItems)
				.OnGenerateRow(this, &SEnhancedInputTab::OnGenerateRow)
				.HeaderRow(HeaderRow)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FEnhancedInputDisplayItem> Item, ESelectInfo::Type)
				{
					SelectedItem = Item;
					RebuildDetails();
				})
			]

			// Shown when all rows are hidden by the active filters.
			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Visibility_Lambda([this]() -> EVisibility
				{
					if (!DisplayItems.IsEmpty())
					{
						return EVisibility::Collapsed;
					}
					const bool bAnyContextFilter = FilterContexts.ContainsByPredicate(
						[](const TWeakObjectPtr<UInputMappingContext>& W) { return W.IsValid(); });
					const bool bAnyActionFilter = FilterActions.ContainsByPredicate(
						[](const TWeakObjectPtr<UInputAction>& W) { return W.IsValid(); });
					const bool bAnyKeyTypeFilter = !bShowKeyboardKeys || !bShowGamepadKeys || !bShowTouchKeys;
					return (bAnyContextFilter || bAnyActionFilter || bAnyKeyTypeFilter)
						? EVisibility::HitTestInvisible
						: EVisibility::Collapsed;
				})
				.Text(LOCTEXT("AllFilteredOut", "All mappings are hidden by the active filters."))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SBox)
				.MaxDesiredHeight(200.f)
				[
					SAssignNew(DetailsBox, SBox)
				]
			]
		]
	];

	RebuildFilterTagsWidget();
	RebuildActionFilterTagsWidget();
	RebuildDetails();
}

void SEnhancedInputTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	RebuildDisplayItems();
}

void SEnhancedInputTab::RemoveFilterContext(int32 Index)
{
	if (FilterContexts.IsValidIndex(Index))
	{
		FilterContexts.RemoveAt(Index);
		RebuildFilterTagsWidget();
	}
}

void SEnhancedInputTab::RemoveFilterAction(int32 Index)
{
	if (FilterActions.IsValidIndex(Index))
	{
		FilterActions.RemoveAt(Index);
		RebuildActionFilterTagsWidget();
	}
}

TSharedRef<SWidget> SEnhancedInputTab::MakeKeyFilterMenuContent()
{
	// Called by the Key column header caret to produce the filter dropdown.
	// Uses IsChecked_Lambda so each open reads the current state from this widget.
	return SNew(SBox)
		.Padding(FMargin(4.f, 2.f))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					return bShowKeyboardKeys ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bShowKeyboardKeys = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowKeyboard", "Keyboard"))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					return bShowGamepadKeys ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bShowGamepadKeys = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowGamepad", "Gamepad"))
				]
			]

			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(2.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]()
				{
					return bShowTouchKeys ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bShowTouchKeys = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowTouch", "Touch"))
				]
			]
		];
}

void SEnhancedInputTab::RebuildFilterTagsWidget()
{
	TSharedRef<SVerticalBox> Content = SNew(SVerticalBox);

	// Count valid (non-null) filter entries for the hint text.
	const int32 ValidCount = FilterContexts.FilterByPredicate(
		[](const TWeakObjectPtr<UInputMappingContext>& W) { return W.IsValid(); }).Num();

	// Header row: status hint + "+" add-slot button
	Content->AddSlot()
	.AutoHeight()
	.Padding(0.f, 0.f, 0.f, 4.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(ValidCount == 0
				? LOCTEXT("FilterNone", "No filter active — showing all contexts")
				: FText::Format(LOCTEXT("FilterActive", "Filtering to {0} context(s)"), FText::AsNumber(ValidCount)))
			.ColorAndOpacity(ValidCount == 0
				? FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f))
				: FSlateColor(FSlateColor::UseForeground()))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("AddFilterTooltip", "Add a context filter slot"))
			.OnClicked_Lambda([this]() -> FReply
			{
				FilterContexts.Add(nullptr);
				RebuildFilterTagsWidget();
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Plus"))
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
			]
		]
	];

	// One asset-picker row per filter slot (matches Details panel TArray<UInputMappingContext*> style).
	for (int32 i = 0; i < FilterContexts.Num(); ++i)
	{
		const int32 CapturedIndex = i;

		Content->AddSlot()
		.AutoHeight()
		.Padding(0.f, 1.f)
		[
			SNew(SHorizontalBox)

			// Array index badge
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(i))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				.MinDesiredWidth(18.f)
			]

			// UE property-style asset picker
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UInputMappingContext::StaticClass())
				.ObjectPath(TAttribute<FString>::CreateLambda([this, CapturedIndex]() -> FString
				{
					if (FilterContexts.IsValidIndex(CapturedIndex))
					{
						if (const UInputMappingContext* IMC = FilterContexts[CapturedIndex].Get())
						{
							return IMC->GetPathName();
						}
					}
					return FString();
				}))
				.OnObjectChanged(FOnSetObject::CreateLambda([this, CapturedIndex](const FAssetData& AssetData)
				{
					if (FilterContexts.IsValidIndex(CapturedIndex))
					{
						FilterContexts[CapturedIndex] = Cast<UInputMappingContext>(AssetData.GetAsset());
						RebuildFilterTagsWidget();
					}
				}))
				.AllowClear(true)
				.DisplayUseSelected(true)
				.DisplayBrowse(true)
			]

			// Delete-slot button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("RemoveFilterTooltip", "Remove this filter slot"))
				.OnClicked_Lambda([this, CapturedIndex]() -> FReply
				{
					RemoveFilterContext(CapturedIndex);
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			]
		];
	}

	FilterTagsContainer->SetContent(Content);
}

void SEnhancedInputTab::RebuildActionFilterTagsWidget()
{
	TSharedRef<SVerticalBox> Content = SNew(SVerticalBox);

	const int32 ValidCount = FilterActions.FilterByPredicate(
		[](const TWeakObjectPtr<UInputAction>& W) { return W.IsValid(); }).Num();

	// Header row: status hint + "+" add-slot button
	Content->AddSlot()
	.AutoHeight()
	.Padding(0.f, 0.f, 0.f, 4.f)
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		.VAlign(VAlign_Center)
		[
			SNew(STextBlock)
			.Text(ValidCount == 0
				? LOCTEXT("ActionFilterNone", "No filter active — showing all actions")
				: FText::Format(LOCTEXT("ActionFilterActive", "Filtering to {0} action(s)"), FText::AsNumber(ValidCount)))
			.ColorAndOpacity(ValidCount == 0
				? FSlateColor(FLinearColor(0.5f, 0.5f, 0.5f))
				: FSlateColor(FSlateColor::UseForeground()))
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "SimpleButton")
			.ToolTipText(LOCTEXT("AddActionFilterTooltip", "Add an action filter slot"))
			.OnClicked_Lambda([this]() -> FReply
			{
				FilterActions.Add(nullptr);
				RebuildActionFilterTagsWidget();
				return FReply::Handled();
			})
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Plus"))
				.DesiredSizeOverride(FVector2D(16.f, 16.f))
			]
		]
	];

	// One asset-picker row per filter slot
	for (int32 i = 0; i < FilterActions.Num(); ++i)
	{
		const int32 CapturedIndex = i;

		Content->AddSlot()
		.AutoHeight()
		.Padding(0.f, 1.f)
		[
			SNew(SHorizontalBox)

			// Array index badge
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			[
				SNew(STextBlock)
				.Text(FText::AsNumber(i))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				.MinDesiredWidth(18.f)
			]

			// UE property-style asset picker
			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			[
				SNew(SObjectPropertyEntryBox)
				.AllowedClass(UInputAction::StaticClass())
				.ObjectPath(TAttribute<FString>::CreateLambda([this, CapturedIndex]() -> FString
				{
					if (FilterActions.IsValidIndex(CapturedIndex))
					{
						if (const UInputAction* IA = FilterActions[CapturedIndex].Get())
						{
							return IA->GetPathName();
						}
					}
					return FString();
				}))
				.OnObjectChanged(FOnSetObject::CreateLambda([this, CapturedIndex](const FAssetData& AssetData)
				{
					if (FilterActions.IsValidIndex(CapturedIndex))
					{
						FilterActions[CapturedIndex] = Cast<UInputAction>(AssetData.GetAsset());
						RebuildActionFilterTagsWidget();
					}
				}))
				.AllowClear(true)
				.DisplayUseSelected(true)
				.DisplayBrowse(true)
			]

			// Delete-slot button
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("RemoveActionFilterTooltip", "Remove this filter slot"))
				.OnClicked_Lambda([this, CapturedIndex]() -> FReply
				{
					RemoveFilterAction(CapturedIndex);
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Delete"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			]
		];
	}

	ActionFilterTagsContainer->SetContent(Content);
}

void SEnhancedInputTab::RebuildDisplayItems()
{
	// Build candidate items into a local array first.
	TArray<TSharedPtr<FEnhancedInputDisplayItem>> NewItems;

	APlayerController* PC = WeakPC.Get();
	ULocalPlayer* LP = PC ? PC->GetLocalPlayer() : nullptr;
	UEnhancedInputLocalPlayerSubsystem* EIS = LP
		? LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>() : nullptr;
	UEnhancedPlayerInput* PlayerInput = EIS ? EIS->GetPlayerInput() : nullptr;

	if (PlayerInput)
	{
		// Collect and sort contexts by priority (highest first).
		TArray<TPair<UInputMappingContext*, int32>> SortedContexts;
		for (TObjectIterator<UInputMappingContext> ContextIt; ContextIt; ++ContextIt)
		{
			UInputMappingContext* IMC = *ContextIt;
			int32 Priority = -1;
			if (IMC && EIS->HasMappingContext(IMC, Priority))
			{
				SortedContexts.Emplace(IMC, Priority);
			}
		}
		SortedContexts.Sort([](const TPair<UInputMappingContext*, int32>& A,
		                       const TPair<UInputMappingContext*, int32>& B)
		{
			return A.Value > B.Value;
		});

		for (const TPair<UInputMappingContext*, int32>& ContextPair : SortedContexts)
		{
			UInputMappingContext* IMC = ContextPair.Key;
			const int32 Priority = ContextPair.Value;

			// Apply context filter.
			const bool bHasValidFilter = FilterContexts.ContainsByPredicate(
				[](const TWeakObjectPtr<UInputMappingContext>& W) { return W.IsValid(); });
			if (bHasValidFilter)
			{
				const bool bPassesFilter = FilterContexts.ContainsByPredicate(
					[IMC](const TWeakObjectPtr<UInputMappingContext>& W) { return W.Get() == IMC; });
				if (!bPassesFilter)
				{
					continue;
				}
			}

			// Context header row
			TSharedPtr<FEnhancedInputDisplayItem> Header = MakeShared<FEnhancedInputDisplayItem>();
			Header->bIsHeader = true;
			Header->ContextName = FText::FromString(IMC->GetName());
			Header->Priority = Priority;
			Header->WeakAsset = IMC;
			NewItems.Add(MoveTemp(Header));

			// Input mode info row
			{
				TSharedPtr<FEnhancedInputDisplayItem> InfoRow = MakeShared<FEnhancedInputDisplayItem>();
				InfoRow->bIsInputModeInfo = true;
				InfoRow->WeakAsset = IMC;

				const EMappingContextInputModeFilterOptions FilterOptions = IMC->GetInputModeFilterOptions();
				InfoRow->FilterOptionsOrdinal   = static_cast<uint8>(FilterOptions);
				InfoRow->RegTrackingModeOrdinal = static_cast<uint8>(IMC->GetRegistrationTrackingMode());
				InfoRow->bShouldApplyModeFilter = IMC->ShouldFilterMappingByInputMode();

				if (FilterOptions == EMappingContextInputModeFilterOptions::UseCustomQuery)
				{
					const FGameplayTagQuery Query = IMC->GetInputModeQuery();
					InfoRow->ModeQueryText = Query.IsEmpty()
						? LOCTEXT("ModeQueryEmpty",   "(empty)")
						: FText::FromString(Query.GetDescription());
				}
				else if (FilterOptions == EMappingContextInputModeFilterOptions::UseProjectDefaultQuery)
				{
					InfoRow->ModeQueryText = LOCTEXT("ModeQueryDefault", "(project default)");
				}
				else
				{
					InfoRow->ModeQueryText = LOCTEXT("ModeQueryNone", "—");
				}

				if (InfoRow->bShouldApplyModeFilter)
				{
					InfoRow->bPassesModeFilter = IMC->GetInputModeQuery().Matches(EIS->GetInputMode());
				}

				NewItems.Add(MoveTemp(InfoRow));
			}

			// Action mapping rows
			TArray<TSharedPtr<FEnhancedInputDisplayItem>> ContextRows;
			for (const FEnhancedActionKeyMapping& Mapping : IMC->GetMappings())
			{
				if (!Mapping.Action || Mapping.bShouldBeIgnored)
				{
					continue;
				}

				// Apply action filter: if any valid action is in the filter list, only show mappings for those actions.
				{
					const bool bHasValidActionFilter = FilterActions.ContainsByPredicate(
						[](const TWeakObjectPtr<UInputAction>& W) { return W.IsValid(); });
					if (bHasValidActionFilter && !FilterActions.ContainsByPredicate(
						[&Mapping](const TWeakObjectPtr<UInputAction>& W) { return W.Get() == Mapping.Action.Get(); }))
					{
						continue;
					}
				}

				// Apply key type filter.
				{
					const bool bIsGamepadKey  = Mapping.Key.IsGamepadKey();
					const bool bIsTouchKey    = Mapping.Key.IsTouch();
					const bool bIsKeyboardKey = !bIsGamepadKey && !bIsTouchKey;
					if ((bIsGamepadKey  && !bShowGamepadKeys)  ||
						(bIsTouchKey    && !bShowTouchKeys)    ||
						(bIsKeyboardKey && !bShowKeyboardKeys))
					{
						continue;
					}
				}

				const FInputActionInstance* Instance = PlayerInput->FindActionInstanceData(Mapping.Action);

				TSharedPtr<FEnhancedInputDisplayItem> Row = MakeShared<FEnhancedInputDisplayItem>();
				Row->bIsHeader = false;
				Row->ActionName = FText::FromString(Mapping.Action->GetName());
				Row->BoundKey = Mapping.Key;
				Row->WeakAsset = Mapping.Action.Get();

				if (Instance)
				{
					Row->TriggerEventOrdinal = static_cast<uint8>(Instance->GetTriggerEvent());
					Row->ValueText = FText::FromString(Instance->GetValue().ToString());
				}
				else
				{
					Row->TriggerEventOrdinal = 0;
					Row->ValueText = LOCTEXT("NoValue", "-");
				}

				ContextRows.Add(MoveTemp(Row));
			}

			ContextRows.Sort([](const TSharedPtr<FEnhancedInputDisplayItem>& A,
			                    const TSharedPtr<FEnhancedInputDisplayItem>& B)
			{
				return A->ActionName.CompareTo(B->ActionName) < 0;
			});

			for (TSharedPtr<FEnhancedInputDisplayItem>& Row : ContextRows)
			{
				NewItems.Add(MoveTemp(Row));
			}
		}
	}

	// Structural comparison: only rebuild the widget tree (RequestListRefresh) when the set of
	// IMCs, actions, or their structural fields changes. If only live values changed
	// (trigger state, current value, filter pass status), update the existing items in-place —
	// the row widgets read those via TAttribute lambdas and will repaint automatically.
	// This keeps the SHyperlink widgets alive so click events can actually fire.
	bool bStructureChanged = (NewItems.Num() != DisplayItems.Num());
	if (!bStructureChanged)
	{
		for (int32 i = 0; i < NewItems.Num() && !bStructureChanged; ++i)
		{
			const FEnhancedInputDisplayItem& N = *NewItems[i];
			const FEnhancedInputDisplayItem& D = *DisplayItems[i];
			bStructureChanged =
				N.bIsHeader              != D.bIsHeader              ||
				N.bIsInputModeInfo       != D.bIsInputModeInfo       ||
				N.Priority               != D.Priority               ||
				N.BoundKey               != D.BoundKey               ||
				N.FilterOptionsOrdinal   != D.FilterOptionsOrdinal   ||
				N.RegTrackingModeOrdinal != D.RegTrackingModeOrdinal ||
				N.WeakAsset              != D.WeakAsset              ||
				N.ContextName.ToString() != D.ContextName.ToString() ||
				N.ActionName.ToString()  != D.ActionName.ToString()  ||
				N.ModeQueryText.ToString() != D.ModeQueryText.ToString();
		}
	}

	if (bStructureChanged)
	{
		DisplayItems = MoveTemp(NewItems);
		ListView->RequestListRefresh();
	}
	else
	{
		// Patch live fields in-place; TAttribute lambdas in the row widgets will repaint.
		for (int32 i = 0; i < NewItems.Num(); ++i)
		{
			DisplayItems[i]->TriggerEventOrdinal    = NewItems[i]->TriggerEventOrdinal;
			DisplayItems[i]->ValueText              = NewItems[i]->ValueText;
			DisplayItems[i]->bShouldApplyModeFilter = NewItems[i]->bShouldApplyModeFilter;
			DisplayItems[i]->bPassesModeFilter      = NewItems[i]->bPassesModeFilter;
		}
		// No RequestListRefresh — existing row widgets survive, hyperlinks remain clickable.
	}
}

TSharedRef<ITableRow> SEnhancedInputTab::OnGenerateRow(TSharedPtr<FEnhancedInputDisplayItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (Item->bIsHeader)
	{
		return SNew(SEnhancedInputContextHeaderRow, OwnerTable)
			.Item(Item);
	}

	if (Item->bIsInputModeInfo)
	{
		return SNew(SEnhancedInputModeInfoRow, OwnerTable)
			.Item(Item);
	}

	return SNew(SEnhancedInputRow, OwnerTable)
		.Item(Item);
}

void SEnhancedInputTab::RebuildDetails()
{
	if (!DetailsBox.IsValid()) { return; }

	if (!SelectedItem.IsValid())
	{
		DetailsBox->SetContent(
			SNew(SBox).Padding(FMargin(8.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoSelection", "Select a row to view details."))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
			]
		);
		return;
	}

	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox);

	auto MakeRow = [](FText Label, TSharedRef<SWidget> Value) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				.MinDesiredWidth(160.f)
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				Value
			];
	};

	auto AddRow = [&](FText Label, TSharedRef<SWidget> Value)
	{
		ScrollBox->AddSlot().Padding(FMargin(8.f, 3.f))[ MakeRow(MoveTemp(Label), Value) ];
	};

	auto AddTextRow = [&](FText Label, FText Value, FLinearColor Color = FLinearColor(0.9f, 0.9f, 0.9f))
	{
		AddRow(MoveTemp(Label), SNew(STextBlock).Text(MoveTemp(Value)).ColorAndOpacity(Color));
	};

	auto MakeAssetWidget = [](FText Name, TWeakObjectPtr<const UObject> WeakAsset) -> TSharedRef<SWidget>
	{
		if (WeakAsset.IsValid())
		{
			return SNew(SHyperlink)
				.Text(Name)
				.Style(&GetYellowHyperlinkStyle())
				.OnNavigate(FSimpleDelegate::CreateLambda([WeakAsset]()
				{
					OpenAssetEditor(WeakAsset);
				}));
		}
		return SNew(STextBlock).Text(Name).ColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.9f));
	};

	if (SelectedItem->bIsHeader || SelectedItem->bIsInputModeInfo)
	{
		const UInputMappingContext* IMC = Cast<const UInputMappingContext>(SelectedItem->WeakAsset.Get());

		// Name + hyperlink
		const FText DisplayName = IMC
			? FText::FromString(IMC->GetName())
			: (SelectedItem->bIsHeader ? SelectedItem->ContextName : LOCTEXT("UnknownIMC", "(unknown)"));
		AddRow(LOCTEXT("DetailIMCName", "Context:"), MakeAssetWidget(DisplayName, SelectedItem->WeakAsset));

		if (IMC)
		{
			AddTextRow(LOCTEXT("DetailIMCPath", "Full Path:"),
				FText::FromString(IMC->GetPathName()),
				FLinearColor(0.5f, 0.5f, 0.5f));
		}

		if (SelectedItem->bIsHeader)
		{
			AddTextRow(LOCTEXT("DetailIMCPriority", "Priority:"), FText::AsNumber(SelectedItem->Priority));
		}

		if (IMC)
		{
			// Filter mode
			const EMappingContextInputModeFilterOptions FilterOptions = IMC->GetInputModeFilterOptions();
			FText FilterText;
			switch (FilterOptions)
			{
			case EMappingContextInputModeFilterOptions::UseCustomQuery:
				FilterText = LOCTEXT("FilterCustom",   "Custom Query");    break;
			case EMappingContextInputModeFilterOptions::DoNotFilter:
				FilterText = LOCTEXT("FilterDisabled", "Don't Filter");    break;
			default:
				FilterText = LOCTEXT("FilterDefault",  "Project Default"); break;
			}
			AddTextRow(LOCTEXT("DetailFilterMode", "Filter Mode:"), FilterText);

			// Mode query
			FText QueryText;
			if (FilterOptions == EMappingContextInputModeFilterOptions::UseCustomQuery)
			{
				const FGameplayTagQuery Query = IMC->GetInputModeQuery();
				QueryText = Query.IsEmpty()
					? LOCTEXT("QueryEmpty", "(empty)")
					: FText::FromString(Query.GetDescription());
			}
			else if (FilterOptions == EMappingContextInputModeFilterOptions::UseProjectDefaultQuery)
			{
				QueryText = LOCTEXT("QueryDefault", "(project default)");
			}
			else
			{
				QueryText = LOCTEXT("QueryNA", "—");
			}
			AddTextRow(LOCTEXT("DetailModeQuery", "Mode Query:"), QueryText, FLinearColor(0.7f, 0.7f, 0.7f));

			// Passes filter
			if (IMC->ShouldFilterMappingByInputMode())
			{
				APlayerController* PC = WeakPC.Get();
				ULocalPlayer* LP = PC ? PC->GetLocalPlayer() : nullptr;
				UEnhancedInputLocalPlayerSubsystem* EIS = LP
					? LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>() : nullptr;
				if (EIS)
				{
					const bool bPasses = IMC->GetInputModeQuery().Matches(EIS->GetInputMode());
					AddTextRow(LOCTEXT("DetailPassesFilter", "Passes Filter:"),
						bPasses ? LOCTEXT("PassesYes", "Yes") : LOCTEXT("PassesNo", "No"),
						bPasses ? FLinearColor(0.2f, 0.85f, 0.2f) : FLinearColor(0.9f, 0.55f, 0.1f));
				}
			}
			else
			{
				AddTextRow(LOCTEXT("DetailPassesFilter", "Passes Filter:"),
					LOCTEXT("PassesNA", "—"), FLinearColor(0.5f, 0.5f, 0.5f));
			}

			// Registration tracking
			const EMappingContextRegistrationTrackingMode RegMode = IMC->GetRegistrationTrackingMode();
			AddTextRow(LOCTEXT("DetailRegTracking", "Reg Tracking:"),
				RegMode == EMappingContextRegistrationTrackingMode::CountRegistrations
					? LOCTEXT("CountRegistrations",     "Count Registrations")
					: LOCTEXT("RegUntracked", "Untracked"));
		}
	}
	else
	{
		// Action row details
		AddRow(LOCTEXT("DetailActionName", "Action:"),
			MakeAssetWidget(SelectedItem->ActionName, SelectedItem->WeakAsset));

		if (SelectedItem->WeakAsset.IsValid())
		{
			AddTextRow(LOCTEXT("DetailActionPath", "Full Path:"),
				FText::FromString(SelectedItem->WeakAsset->GetPathName()),
				FLinearColor(0.5f, 0.5f, 0.5f));
		}

		AddTextRow(LOCTEXT("DetailBoundKey", "Bound Key:"),
			FText::FromString(SelectedItem->BoundKey.ToString()),
			FLinearColor(0.7f, 0.7f, 0.7f));

		AddRow(LOCTEXT("DetailTriggerState", "Trigger State:"),
			SNew(STextBlock)
			.Text(GetTriggerEventName(SelectedItem->TriggerEventOrdinal))
			.ColorAndOpacity(GetTriggerEventColor(SelectedItem->TriggerEventOrdinal)));

		AddTextRow(LOCTEXT("DetailCurrentValue", "Current Value:"), SelectedItem->ValueText);
	}

	DetailsBox->SetContent(ScrollBox);
}

#undef LOCTEXT_NAMESPACE
