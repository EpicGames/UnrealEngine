// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerController.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

class UInputAction;
class UInputMappingContext;
class SEditableTextBox;

// A single row in the Enhanced Input display: a context header, an IMC mode info row, or an action mapping row.
struct FEnhancedInputDisplayItem
{
	bool bIsHeader = false;
	bool bIsInputModeInfo = false;

	// Header fields
	FText ContextName;
	int32 Priority = 0;

	// Input mode info row fields (appears once per IMC, directly after its header)
	// Enums stored as uint8 to avoid pulling InputMappingContext.h into this header.
	uint8 FilterOptionsOrdinal = 0;    // EMappingContextInputModeFilterOptions
	uint8 RegTrackingModeOrdinal = 0;  // EMappingContextRegistrationTrackingMode
	FText ModeQueryText;
	bool bShouldApplyModeFilter = false;  // False when DoNotFilter or global filtering is disabled
	bool bPassesModeFilter = false;       // Only meaningful when bShouldApplyModeFilter is true

	// Action mapping row fields
	FText ActionName;
	FKey BoundKey;
	uint8 TriggerEventOrdinal = 0;  // ETriggerEvent stored as uint8 to avoid including InputAction.h in header
	FText ValueText;

	// Asset reference for hyperlink navigation and details panel (IMC for headers/info rows, IA for action rows).
	TWeakObjectPtr<const UObject> WeakAsset;
};

// Tab showing active Enhanced Input mapping contexts, their actions, bound keys,
// trigger states and values for the selected player controller. Refreshes every frame.
class SEnhancedInputTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SEnhancedInputTab) {}
		SLATE_ARGUMENT(TWeakObjectPtr<APlayerController>, PlayerController)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SEnhancedInputTab();
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SetPlayerController(APlayerController* PC);

private:
	void RebuildDisplayItems();
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FEnhancedInputDisplayItem> Item, const TSharedRef<STableViewBase>& OwnerTable);

	// Context filter helpers
	void RemoveFilterContext(int32 Index);
	void RebuildFilterTagsWidget();

	// Action filter helpers
	void RemoveFilterAction(int32 Index);
	void RebuildActionFilterTagsWidget();

	// Key type filter — dropdown menu content for the Key column header
	TSharedRef<SWidget> MakeKeyFilterMenuContent();

	void RebuildDetails();

	TWeakObjectPtr<APlayerController> WeakPC;
	TArray<TSharedPtr<FEnhancedInputDisplayItem>> DisplayItems;
	TSharedPtr<SListView<TSharedPtr<FEnhancedInputDisplayItem>>> ListView;

	// Contexts to show; empty = show all.
	TArray<TWeakObjectPtr<UInputMappingContext>> FilterContexts;
	TSharedPtr<SBox> FilterTagsContainer;

	// Actions to show; empty = show all.
	TArray<TWeakObjectPtr<UInputAction>> FilterActions;
	TSharedPtr<SBox> ActionFilterTagsContainer;

	// Key type visibility flags (default all visible)
	bool bShowKeyboardKeys = true;
	bool bShowGamepadKeys  = true;
	bool bShowTouchKeys    = true;

	// Mappable Key Profiles section
	TSharedPtr<SEditableTextBox> ProfileIdInput;

	// Selection and details panel
	TSharedPtr<FEnhancedInputDisplayItem> SelectedItem;
	TSharedPtr<SBox> DetailsBox;
};
