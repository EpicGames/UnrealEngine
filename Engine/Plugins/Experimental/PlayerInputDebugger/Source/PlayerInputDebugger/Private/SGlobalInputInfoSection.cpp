// Copyright Epic Games, Inc. All Rights Reserved.

#include "SGlobalInputInfoSection.h"

#include "Components/InputComponent.h"
#include "CommonInputSubsystem.h"
#include "EnhancedInputSubsystems.h"
#include "Engine/GameViewportClient.h"
#include "Engine/LocalPlayer.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerInput.h"
#include "Input/CommonUIActionRouterBase.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SGlobalInputInfoSection"

// ── Stack row widget ──────────────────────────────────────────────────────────

namespace
{

class SGlobalInputStackRow : public SMultiColumnTableRow<TSharedPtr<FGlobalStackRow>>
{
public:
	SLATE_BEGIN_ARGS(SGlobalInputStackRow) {}
		SLATE_ARGUMENT(TSharedPtr<FGlobalStackRow>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		SMultiColumnTableRow<TSharedPtr<FGlobalStackRow>>::Construct(
			FSuperRowType::FArguments().Padding(FMargin(2.f, 1.f)),
			InOwnerTableView
		);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(Item.IsValid());

		static const FLinearColor BlockingColor(1.0f, 0.6f, 0.1f);

		if (ColumnName == TEXT("StackPriority"))
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(Item->Priority))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(Item->bIsBlockingComponent ? BlockingColor : FLinearColor::White);
		}

		if (ColumnName == TEXT("StackComponent"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->ComponentName))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(Item->bIsBlockingComponent ? BlockingColor : FLinearColor::White);
		}

		if (ColumnName == TEXT("StackOwner"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->OwnerName))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(Item->bIsBlockingComponent ? BlockingColor : FLinearColor(0.6f, 0.6f, 0.6f));
		}

		return SNew(STextBlock).Text(FText::GetEmpty());
	}

private:
	TSharedPtr<FGlobalStackRow> Item;
};

} // anonymous namespace

// ── Construct ─────────────────────────────────────────────────────────────────

void SGlobalInputInfoSection::Construct(const FArguments& InArgs)
{
	SetCanTick(true);

	// Helper: one label + live-text value row for the config section.
	auto MakeConfigRow = [](const FText& Label, TAttribute<FText> Value, float TopPad = 1.f, float BotPad = 1.f) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				.MinDesiredWidth(160.f)
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock).Text(Value)
			];
	};

	// ── Input Configuration body ──────────────────────────────────────────────
	TSharedRef<SVerticalBox> ConfigBody = SNew(SVerticalBox)

		// Player Controller sub-header
		+ SVerticalBox::Slot().AutoHeight().Padding(4.f, 6.f, 4.f, 4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("PCSubHeader", "Player Controller"))
			.Font(FAppStyle::GetFontStyle("BoldFont"))
			.ColorAndOpacity(FLinearColor(0.9f, 0.8f, 0.3f))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("InputModeLabel", "Input Mode:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					return FText::FromString(PC->GetCurrentInputModeDebugString());
				}))
		]
		
		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("EnhancedInputModeLabel", "Enhanced Input Mode:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("UnknownEIInputMode", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("UnknownEIInputModeNoLP", "—"); }
					UEnhancedInputLocalPlayerSubsystem* EIS = LP->GetSubsystem<UEnhancedInputLocalPlayerSubsystem>();
					if (!EIS) { return LOCTEXT("UnknownEIS", "—"); }
					return FText::FromString(EIS->GetInputMode().ToString());
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("MouseLockLabel", "Mouse Lock Mode:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UGameViewportClient* GVC = LP->ViewportClient;
					if (!GVC) { return LOCTEXT("ConfigNA", "—"); }
					return UEnum::GetDisplayValueAsText(GVC->GetMouseLockMode());
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("MouseCaptureLabel", "Mouse Capture Mode:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UGameViewportClient* GVC = LP->ViewportClient;
					if (!GVC) { return LOCTEXT("ConfigNA", "—"); }
					return UEnum::GetDisplayValueAsText(GVC->GetMouseCaptureMode());
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("MousePosLabel", "Mouse Position:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UGameViewportClient* GVC = LP->ViewportClient;
					if (!GVC) { return LOCTEXT("ConfigNA", "—"); }
					FVector2D MousePos(EForceInit::ForceInitToZero);
					if (!GVC->GetMousePosition(MousePos))
					{
						return LOCTEXT("MouseOffScreen", "(off-screen)");
					}
					return FText::FromString(FString::Printf(TEXT("%.0f, %.0f"), MousePos.X, MousePos.Y));
				}))
		]

		// UI sub-header
		+ SVerticalBox::Slot().AutoHeight().Padding(4.f, 8.f, 4.f, 4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("UISubHeader", "UI"))
			.Font(FAppStyle::GetFontStyle("BoldFont"))
			.ColorAndOpacity(FLinearColor(0.9f, 0.8f, 0.3f))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("CommonInputTypeLabel", "Input Type:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UCommonInputSubsystem* CIS = LP->GetSubsystem<UCommonInputSubsystem>();
					if (!CIS) { return LOCTEXT("ConfigNA", "—"); }
					return UEnum::GetDisplayValueAsText(CIS->GetCurrentInputType());
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("GamepadNameLabel", "Gamepad Name:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UCommonInputSubsystem* CIS = LP->GetSubsystem<UCommonInputSubsystem>();
					if (!CIS) { return LOCTEXT("ConfigNA", "—"); }
					const FName GamepadName = CIS->GetCurrentGamepadName();
					return GamepadName.IsNone()
						? LOCTEXT("NoGamepad", "(none)")
						: FText::FromName(GamepadName);
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("UIInputModeLabel", "Common UI Input Mode:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					APlayerController* PC = WeakPC.Get();
					if (!PC) { return LOCTEXT("ConfigNA", "—"); }
					ULocalPlayer* LP = PC->GetLocalPlayer();
					if (!LP) { return LOCTEXT("ConfigNA", "—"); }
					UCommonUIActionRouterBase* Router = LP->GetSubsystem<UCommonUIActionRouterBase>();
					if (!Router) { return LOCTEXT("ConfigNA", "—"); }
					return UEnum::GetDisplayValueAsText(Router->GetActiveInputMode());
				}))
		]

		+ SVerticalBox::Slot().AutoHeight()
		[
			MakeConfigRow(LOCTEXT("UIInputConfigLabel", "Active Input Config:"),
				TAttribute<FText>::CreateLambda([this]() -> FText
				{
					return CachedInputConfigString.IsEmpty()
						? LOCTEXT("ConfigNA", "—")
						: FText::FromString(CachedInputConfigString);
				}), 1.f, 6.f)
		];

	// ── Input Component Stack body ────────────────────────────────────────────
	TSharedRef<SBox> StackBody = SNew(SBox)
		.MaxDesiredHeight(300.f)
		[
			SAssignNew(StackListView, SListView<TSharedPtr<FGlobalStackRow>>)
			.ListItemsSource(&StackRows)
			.OnGenerateRow(this, &SGlobalInputInfoSection::OnGenerateStackRow)
			.HeaderRow(
				SNew(SHeaderRow)
				+ SHeaderRow::Column("StackPriority")
				.DefaultLabel(LOCTEXT("ColStackPriority", "Priority"))
				.ManualWidth(70.f)

				+ SHeaderRow::Column("StackComponent")
				.DefaultLabel(LOCTEXT("ColStackComponent", "Input Component"))
				.ManualWidth(260.f)

				+ SHeaderRow::Column("StackOwner")
				.DefaultLabel(LOCTEXT("ColStackOwner", "Owner"))
				.ManualWidth(260.f)
			)
			.SelectionMode(ESelectionMode::None)
		];

	// ── Root layout ───────────────────────────────────────────────────────────
	ChildSlot
	[
		SNew(SExpandableArea)
		.AreaTitle(LOCTEXT("SectionTitle", "Input Overview"))
		.InitiallyCollapsed(false)
		.BodyContent()
		[
			SNew(SVerticalBox)

			// Focused widget row
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(4.f, 4.f, 4.f, 4.f)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.f, 0.f, 6.f, 0.f)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("FocusedWidgetLabel", "Focused widget:"))
					.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text_Lambda([]() -> FText
					{
						if (!FSlateApplication::IsInitialized())
						{
							return FText::GetEmpty();
						}
						const TSharedPtr<SWidget> Focused = FSlateApplication::Get().GetUserFocusedWidget(0);
						if (!Focused)
						{
							return LOCTEXT("FocusNone", "(none)");
						}
						return FText::FromString(Focused->GetType().ToString());
					})
				]
			]

			// Input Configuration
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("InputConfigTitle", "Input Configuration"))
				.InitiallyCollapsed(true)
				.BodyContent()
				[
					ConfigBody
				]
			]

			// Input Component Stack
			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SExpandableArea)
				.AreaTitle(LOCTEXT("InputStackTitle", "Input Component Stack"))
				.InitiallyCollapsed(true)
				.BodyContent()
				[
					StackBody
				]
			]
		]
	];
}

SGlobalInputInfoSection::~SGlobalInputInfoSection()
{
	if (UCommonUIActionRouterBase* Router = WeakRouter.Get())
	{
		Router->OnActiveInputConfigChanged().Remove(InputConfigChangedHandle);
	}
	InputConfigChangedHandle.Reset();
}

void SGlobalInputInfoSection::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	RebuildStackRows();
}

void SGlobalInputInfoSection::SetPlayerController(APlayerController* PC)
{
	// Unsubscribe from the previous router.
	if (UCommonUIActionRouterBase* OldRouter = WeakRouter.Get())
	{
		OldRouter->OnActiveInputConfigChanged().Remove(InputConfigChangedHandle);
		InputConfigChangedHandle.Reset();
	}
	WeakRouter.Reset();
	CachedInputConfigString.Empty();

	WeakPC = PC;

	// Subscribe to the new router.
	if (PC)
	{
		if (ULocalPlayer* LP = PC->GetLocalPlayer())
		{
			if (UCommonUIActionRouterBase* Router = LP->GetSubsystem<UCommonUIActionRouterBase>())
			{
				WeakRouter = Router;
				InputConfigChangedHandle = Router->OnActiveInputConfigChanged().AddLambda(
					[this](const FUIInputConfig& NewConfig)
					{
						CachedInputConfigString = NewConfig.ToString();
					});
			}
		}
	}
}

void SGlobalInputInfoSection::RebuildStackRows()
{
	StackRows.Empty();

	APlayerController* PC = WeakPC.Get();
	if (PC)
	{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		for (const TWeakObjectPtr<const UInputComponent>& WeakIC : PC->Debug_GetMostRecentInputStack())
		{
			if (const UInputComponent* IC = WeakIC.Get())
			{
				TSharedPtr<FGlobalStackRow> Row = MakeShared<FGlobalStackRow>();
				Row->Priority = IC->Priority;
				Row->ComponentName = IC->GetName();
				Row->bIsBlockingComponent = IC->bBlockInput;

				if (const AActor* Owner = IC->GetOwner())
				{
					Row->OwnerName = Owner->GetActorNameOrLabel();
				}
				else if (const UObject* Outer = IC->GetOuter())
				{
					Row->OwnerName = Outer->GetName();
				}

				StackRows.Add(MoveTemp(Row));
			}
		}
#endif	// #if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	}

	StackListView->RequestListRefresh();
}

TSharedRef<ITableRow> SGlobalInputInfoSection::OnGenerateStackRow(TSharedPtr<FGlobalStackRow> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SGlobalInputStackRow, OwnerTable)
		.Item(Item);
}

#undef LOCTEXT_NAMESPACE
