// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerController.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

class UCommonUIActionRouterBase;

// Stack row used by the global input overview.
struct FGlobalStackRow
{
	int32 Priority = 0;
	FString ComponentName;
	FString OwnerName;
	bool bIsBlockingComponent = false;
};

// Persistent collapsible section shown at the top of the debugger window
// regardless of which tab is active. Displays:
//   • Currently focused Slate widget
//   • Input Configuration (input mode, mouse settings, CommonUI state)
//   • Input Component Stack
class SGlobalInputInfoSection : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SGlobalInputInfoSection) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SGlobalInputInfoSection();
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void SetPlayerController(APlayerController* PC);

private:
	void RebuildStackRows();
	TSharedRef<ITableRow> OnGenerateStackRow(TSharedPtr<FGlobalStackRow> Item, const TSharedRef<STableViewBase>& OwnerTable);

	TWeakObjectPtr<APlayerController> WeakPC;

	// Input Component Stack section
	TArray<TSharedPtr<FGlobalStackRow>> StackRows;
	TSharedPtr<SListView<TSharedPtr<FGlobalStackRow>>> StackListView;

	// Active Input Config — cached from UCommonUIActionRouterBase::OnActiveInputConfigChanged()
	FString CachedInputConfigString;
	FDelegateHandle InputConfigChangedHandle;
	TWeakObjectPtr<UCommonUIActionRouterBase> WeakRouter;
};
