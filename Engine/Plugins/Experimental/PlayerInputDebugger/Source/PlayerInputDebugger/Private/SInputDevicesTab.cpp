// Copyright Epic Games, Inc. All Rights Reserved.

#include "SInputDevicesTab.h"

#include "GameFramework/InputDeviceSubsystem.h"
#include "GameFramework/InputSettings.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GenericPlatform/InputDeviceMappingPolicy.h"
#include "GenericPlatform/InputDeviceRegistry.h"
#include "Misc/CoreMiscDefines.h"
#include "SEnumCombo.h"
#include "Settings/LevelEditorPlaySettings.h"
#include "Styling/AppStyle.h"
#include "UObject/UnrealType.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SInputDevicesTab"

static FText GetMappingPolicyText(EInputDeviceMappingPolicy Policy)
{
	switch (Policy)
	{
		case EInputDeviceMappingPolicy::UseManagedPlatformLogin:
			return LOCTEXT("PolicyManaged",   "UseManagedPlatformLogin");
		case EInputDeviceMappingPolicy::PrimaryUserSharesKeyboardAndFirstGamepad:
			return LOCTEXT("PolicyPrimary",   "PrimaryUserSharesKeyboardAndFirstGamepad");
		case EInputDeviceMappingPolicy::CreateUniquePlatformUserForEachDevice:
			return LOCTEXT("PolicyUnique",    "CreateUniquePlatformUserForEachDevice");
		case EInputDeviceMappingPolicy::MapAllDevicesToPrimaryUser:
			return LOCTEXT("PolicyMapAll",    "MapAllDevicesToPrimaryUser");
		default:
			return LOCTEXT("PolicyInvalid",   "Invalid");
	}
}

static FName GetDeviceIconBrushName(EHardwareDevicePrimaryType PrimaryType)
{
	switch (PrimaryType)
	{
		case EHardwareDevicePrimaryType::KeyboardAndMouse: return "GraphEditor.KeyEvent_16x";
		case EHardwareDevicePrimaryType::Gamepad:          return "GraphEditor.PadEvent_16x";
		case EHardwareDevicePrimaryType::Touch:            return "GraphEditor.TouchEvent_16x";
		case EHardwareDevicePrimaryType::MotionTracking:   return "GraphEditor.PadEvent_16x";
		case EHardwareDevicePrimaryType::RacingWheel:      return "GraphEditor.PadEvent_16x";
		case EHardwareDevicePrimaryType::FlightStick:      return "GraphEditor.PadEvent_16x";
		default:                                           return NAME_None;
	}
}

static FText BuildSupportedFeaturesText(int32 Mask)
{
	if (Mask == EHardwareDeviceSupportedFeatures::Unspecified)
	{
		return NSLOCTEXT("SInputDevicesTab", "FeaturesNone", "(none)");
	}

	static const TPair<EHardwareDeviceSupportedFeatures::Type, FStringView> FeatureList[] =
	{
		{ EHardwareDeviceSupportedFeatures::Keypress,           TEXTVIEW("Keypress") },
		{ EHardwareDeviceSupportedFeatures::Pointer,            TEXTVIEW("Pointer") },
		{ EHardwareDeviceSupportedFeatures::Gamepad,            TEXTVIEW("Gamepad") },
		{ EHardwareDeviceSupportedFeatures::Touch,              TEXTVIEW("Touch") },
		{ EHardwareDeviceSupportedFeatures::Camera,             TEXTVIEW("Camera") },
		{ EHardwareDeviceSupportedFeatures::MotionTracking,     TEXTVIEW("Motion Tracking") },
		{ EHardwareDeviceSupportedFeatures::Lights,             TEXTVIEW("Lights") },
		{ EHardwareDeviceSupportedFeatures::TriggerHaptics,     TEXTVIEW("Trigger Haptics") },
		{ EHardwareDeviceSupportedFeatures::ForceFeedback,      TEXTVIEW("Force Feedback") },
		{ EHardwareDeviceSupportedFeatures::AudioBasedVibrations, TEXTVIEW("Audio Vibrations") },
		{ EHardwareDeviceSupportedFeatures::Acceleration,       TEXTVIEW("Acceleration") },
		{ EHardwareDeviceSupportedFeatures::Virtual,            TEXTVIEW("Virtual") },
		{ EHardwareDeviceSupportedFeatures::Microphone,         TEXTVIEW("Microphone") },
		{ EHardwareDeviceSupportedFeatures::Orientation,        TEXTVIEW("Orientation") },
		{ EHardwareDeviceSupportedFeatures::Guitar,             TEXTVIEW("Guitar") },
		{ EHardwareDeviceSupportedFeatures::Drums,              TEXTVIEW("Drums") },
		{ EHardwareDeviceSupportedFeatures::CustomA,            TEXTVIEW("Custom A") },
		{ EHardwareDeviceSupportedFeatures::CustomB,            TEXTVIEW("Custom B") },
		{ EHardwareDeviceSupportedFeatures::CustomC,            TEXTVIEW("Custom C") },
		{ EHardwareDeviceSupportedFeatures::CustomD,            TEXTVIEW("Custom D") },
	};

	FString Out;
	for (const auto& [Flag, Name] : FeatureList)
	{
		if (Mask & Flag)
		{
			if (!Out.IsEmpty()) { Out += TEXT(", "); }
			Out += Name;
		}
	}
	return FText::FromString(Out);
}

// ── Column IDs ────────────────────────────────────────────────────────────────
namespace DevicesTab
{
	static const FName Col_Device   = "Device";
	static const FName Col_Type     = "Type";
	static const FName Col_Hardware = "Hardware";
	static const FName Col_State    = "State";
}

// ── State / type helpers (free functions used by both tab and row class) ──────

static FText GetConnectionStateText(EInputDeviceConnectionState State)
{
	switch (State)
	{
		case EInputDeviceConnectionState::Connected:    return LOCTEXT("StateConnected",    "Connected");
		case EInputDeviceConnectionState::Disconnected: return LOCTEXT("StateDisconnected", "Disconnected");
		case EInputDeviceConnectionState::Unknown:      return LOCTEXT("StateUnknown",      "Unknown");
		default:                                        return LOCTEXT("StateInvalid",      "Invalid");
	}
}

static FSlateColor GetConnectionStateColor(EInputDeviceConnectionState State)
{
	switch (State)
	{
		case EInputDeviceConnectionState::Connected:    return FLinearColor(0.2f, 0.9f, 0.2f);
		case EInputDeviceConnectionState::Disconnected: return FLinearColor(0.9f, 0.2f, 0.2f);
		case EInputDeviceConnectionState::Unknown:      return FLinearColor(0.9f, 0.5f, 0.1f);
		default:                                        return FLinearColor(0.9f, 0.1f, 0.1f);
	}
}

static FText GetDeviceTypeText(const FHardwareDeviceIdentifier& HardwareId)
{
	switch (HardwareId.PrimaryDeviceType)
	{
		case EHardwareDevicePrimaryType::KeyboardAndMouse: return LOCTEXT("TypeKBM",           "Keyboard & Mouse");
		case EHardwareDevicePrimaryType::Gamepad:          return LOCTEXT("TypeGamepad",        "Gamepad");
		case EHardwareDevicePrimaryType::Touch:            return LOCTEXT("TypeTouch",          "Touch");
		case EHardwareDevicePrimaryType::MotionTracking:   return LOCTEXT("TypeMotionTracking", "Motion Tracking");
		case EHardwareDevicePrimaryType::RacingWheel:      return LOCTEXT("TypeRacingWheel",    "Racing Wheel");
		case EHardwareDevicePrimaryType::FlightStick:      return LOCTEXT("TypeFlightStick",    "Flight Stick");
		case EHardwareDevicePrimaryType::Camera:           return LOCTEXT("TypeCamera",         "Camera");
		case EHardwareDevicePrimaryType::Instrument:       return LOCTEXT("TypeInstrument",     "Instrument");
		case EHardwareDevicePrimaryType::CustomTypeA:      return LOCTEXT("TypeCustomA",        "Custom A");
		case EHardwareDevicePrimaryType::CustomTypeB:      return LOCTEXT("TypeCustomB",        "Custom B");
		case EHardwareDevicePrimaryType::CustomTypeC:      return LOCTEXT("TypeCustomC",        "Custom C");
		case EHardwareDevicePrimaryType::CustomTypeD:      return LOCTEXT("TypeCustomD",        "Custom D");
		case EHardwareDevicePrimaryType::Unspecified:
		default:                                           return LOCTEXT("TypeUnspec",         "Unspecified");
	}
}

// ── Per-device list row ───────────────────────────────────────────────────────

namespace
{

class SDeviceTableRow : public SMultiColumnTableRow<TSharedPtr<FDeviceListItem>>
{
public:
	SLATE_BEGIN_ARGS(SDeviceTableRow) {}
		SLATE_ARGUMENT(TSharedPtr<FDeviceListItem>, Item)
		SLATE_ARGUMENT(FKnownDevice, Device)
		SLATE_ARGUMENT(bool, bIsSimulated)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
	{
		Item        = InArgs._Item;
		Device      = InArgs._Device;
		bIsSimulated = InArgs._bIsSimulated;
		SMultiColumnTableRow<TSharedPtr<FDeviceListItem>>::Construct(
			FSuperRowType::FArguments().Padding(FMargin(4.f, 2.f)), InOwnerTable);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		// ── Device (icon + index) ────────────────────────────────────────────
		if (ColumnName == DevicesTab::Col_Device)
		{
			static const FLinearColor SimColor(1.0f, 0.6f, 0.1f);
			const FName TypeIconBrush = GetDeviceIconBrushName(Device.HardwareId.PrimaryDeviceType);
			// When simulated: show the debug icon tinted orange in place of the device type icon.
			const FName ActiveIconBrush = bIsSimulated ? FName("Debug") : TypeIconBrush;
			const bool bShowIcon = bIsSimulated || TypeIconBrush != NAME_None;

			return SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0.f, 0.f, 6.f, 0.f)
				.VAlign(VAlign_Center)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(ActiveIconBrush))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.ColorAndOpacity(bIsSimulated ? SimColor : FLinearColor::White)
					.Visibility(bShowIcon ? EVisibility::Visible : EVisibility::Collapsed)
				]

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("DeviceId", "Device {0}"), FText::AsNumber(Item->DeviceId.GetId())))
					.Font(FAppStyle::GetFontStyle("NormalFont"))
					.ColorAndOpacity(bIsSimulated ? SimColor : FLinearColor::White)
				];
		}

		// ── Type ──────────────────────────────────────────────────────────────
		if (ColumnName == DevicesTab::Col_Type)
		{
			return SNew(STextBlock)
				.Text(GetDeviceTypeText(Device.HardwareId))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f));
		}

		// ── Hardware FNames (stacked) ─────────────────────────────────────────
		if (ColumnName == DevicesTab::Col_Hardware)
		{
			return SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(Device.HardwareId.InputClassName.IsNone()
						? LOCTEXT("UnknownInputClass", "(unknown class)")
						: FText::FromName(Device.HardwareId.InputClassName))
					.Font(FAppStyle::GetFontStyle("SmallFont"))
					.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(STextBlock)
					.Text(Device.HardwareId.HardwareDeviceIdentifier.IsNone()
						? LOCTEXT("UnknownHardware", "(unknown hardware)")
						: FText::FromName(Device.HardwareId.HardwareDeviceIdentifier))
					.Font(FAppStyle::GetFontStyle("SmallFont"))
					.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				];
		}

		// ── Connection State ──────────────────────────────────────────────────
		if (ColumnName == DevicesTab::Col_State)
		{
			return SNew(STextBlock)
				.Text(GetConnectionStateText(Device.LastState))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(GetConnectionStateColor(Device.LastState));
		}

		return SNullWidget::NullWidget;
	}

private:
	TSharedPtr<FDeviceListItem> Item;
	FKnownDevice Device;
	bool bIsSimulated = false;
};

} // anonymous namespace

// ─────────────────────────────────────────────────────────────────────────────

void SInputDevicesTab::Construct(const FArguments& InArgs)
{
	SetCanTick(false);

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.f, 8.f, 8.f, 4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("Header", "Connected Input Devices"))
			.Font(FAppStyle::GetFontStyle("HeadingExtraSmall"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.f, 0.f, 8.f, 8.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 6.f, 0.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PolicyLabel", "Device Mapping Policy:"))
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Lambda([]() -> FText
				{
					if (const UInputPlatformSettings* Settings = UInputPlatformSettings::Get())
					{
						return GetMappingPolicyText(Settings->DeviceMappingPolicy);
					}
					return LOCTEXT("PolicyUnknown", "Unknown");
				})
			]
		]

		// Simulated device mapping policy controls
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(8.f, 0.f, 8.f, 8.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([]() -> ECheckBoxState
				{
					const ULevelEditorPlaySettings* Settings = GetDefault<ULevelEditorPlaySettings>();
					return Settings->IsSimulateDeviceMappingPolicy() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([](ECheckBoxState NewState)
				{
					ULevelEditorPlaySettings* Settings = GetMutableDefault<ULevelEditorPlaySettings>();
					if (FBoolProperty* Prop = FindFProperty<FBoolProperty>(ULevelEditorPlaySettings::StaticClass(),
						FName(TEXT("bSimulateDeviceMappingPolicy"))))
					{
						Prop->SetPropertyValue_InContainer(Settings, NewState == ECheckBoxState::Checked);
						FPropertyChangedEvent Event(Prop, EPropertyChangeType::ValueSet);
						Settings->PostEditChangeProperty(Event);
						Settings->SaveConfig();
					}
				})
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SimulatePolicyLabel", "Simulate Device Mapping Policy:"))
					.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SEnumComboBox, StaticEnum<EInputDeviceMappingPolicy>())
				.CurrentValue_Lambda([]() -> int32
				{
					return (int32)GetDefault<ULevelEditorPlaySettings>()->GetSimulatedDeviceMappingPolicy();
				})
				.OnEnumSelectionChanged_Lambda([](int32 NewValue, ESelectInfo::Type)
				{
					ULevelEditorPlaySettings* Settings = GetMutableDefault<ULevelEditorPlaySettings>();
					if (FEnumProperty* Prop = FindFProperty<FEnumProperty>(ULevelEditorPlaySettings::StaticClass(),
						FName(TEXT("SimulatedDeviceMappingPolicy"))))
					{
						void* ValuePtr = Prop->ContainerPtrToValuePtr<void>(Settings);
						Prop->GetUnderlyingProperty()->SetIntPropertyValue(ValuePtr, (int64)NewValue);
						FPropertyChangedEvent Event(Prop, EPropertyChangeType::ValueSet);
						Settings->PostEditChangeProperty(Event);
						Settings->SaveConfig();
					}
				})
				.IsEnabled_Lambda([]() -> bool
				{
					return GetDefault<ULevelEditorPlaySettings>()->IsSimulateDeviceMappingPolicy();
				})
			]
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SOverlay)

			+ SOverlay::Slot()
			[
				SAssignNew(DeviceListView, SListView<TSharedPtr<FDeviceListItem>>)
				.ListItemsSource(&DeviceDisplayItems)
				.OnGenerateRow(this, &SInputDevicesTab::OnGenerateDeviceRow)
				.SelectionMode(ESelectionMode::Single)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FDeviceListItem> Item, ESelectInfo::Type)
				{
					if (!Item.IsValid() || Item->bIsUserHeader) { return; }
					SelectedDeviceId = Item->DeviceId;
					RebuildDetails();
				})
				.HeaderRow(
					SNew(SHeaderRow)
					+ SHeaderRow::Column(DevicesTab::Col_Device)
					.DefaultLabel(LOCTEXT("ColDevice", "Device"))
					.ManualWidth(160.f)

					+ SHeaderRow::Column(DevicesTab::Col_Type)
					.DefaultLabel(LOCTEXT("ColType", "Type"))
					.ManualWidth(130.f)

					+ SHeaderRow::Column(DevicesTab::Col_Hardware)
					.DefaultLabel(LOCTEXT("ColHardware", "Hardware Names"))
					.FillWidth(1.f)

					+ SHeaderRow::Column(DevicesTab::Col_State)
					.DefaultLabel(LOCTEXT("ColState", "State"))
					.ManualWidth(110.f)
				)
			]

			+ SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoUsers", "No active platform users."))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
				.Visibility_Lambda([this]()
				{
					return DeviceDisplayItems.IsEmpty() ? EVisibility::HitTestInvisible : EVisibility::Collapsed;
				})
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SBox)
				.MaxDesiredHeight(200.f)
				[
					SAssignNew(DetailsBox, SBox)
				]
			]
		]
	];

	ConnectionChangedHandle = IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().AddRaw(
		this, &SInputDevicesTab::OnDeviceConnectionChanged);

	PairingChangeHandle = IPlatformInputDeviceMapper::Get().GetOnInputDevicePairingChange().AddRaw(
		this, &SInputDevicesTab::OnDevicePairingChanged);

	if (UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
	{
		HardwareDeviceChangedHandle = DeviceSubsystem->OnInputHardwareDeviceChangedNative.AddRaw(
			this, &SInputDevicesTab::OnHardwareDeviceChanged);
	}

	// Seed KnownDevices from currently active devices.
	{
		IPlatformInputDeviceMapper& Mapper = IPlatformInputDeviceMapper::Get();
		UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get();

		TArray<FPlatformUserId> ActiveUsers;
		Mapper.GetAllActiveUsers(ActiveUsers);

		for (const FPlatformUserId& UserId : ActiveUsers)
		{
			TArray<FInputDeviceId> Devices;
			Mapper.GetAllInputDevicesForUser(UserId, Devices);

			for (const FInputDeviceId& DeviceId : Devices)
			{
				TSharedPtr<FKnownDevice> Entry = MakeShared<FKnownDevice>();
				Entry->UserId = UserId;
				Entry->LastState = Mapper.GetInputDeviceConnectionState(DeviceId);
				if (DeviceSubsystem)
				{
					Entry->HardwareId = DeviceSubsystem->GetInputDeviceHardwareIdentifier(DeviceId);
				}
				KnownDevices.Add(DeviceId, MoveTemp(Entry));
			}
		}
	}

	RebuildDeviceList();

	// Populate available hardware device identifiers for the simulated descriptor picker.
	{
		for (const FHardwareDeviceIdentifier& HWId : UInputPlatformSettings::GetAllHardwareDeviceIdentifiers())
		{
			AvailableHardwareDevices.Add(MakeShared<FHardwareDeviceIdentifier>(HWId));
		}
	}

	RebuildDetails();
}

SInputDevicesTab::~SInputDevicesTab()
{
	IPlatformInputDeviceMapper::Get().GetOnInputDeviceConnectionChange().Remove(ConnectionChangedHandle);
	IPlatformInputDeviceMapper::Get().GetOnInputDevicePairingChange().Remove(PairingChangeHandle);

	if (UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
	{
		DeviceSubsystem->OnInputHardwareDeviceChangedNative.Remove(HardwareDeviceChangedHandle);
	}
}

void SInputDevicesTab::RebuildDeviceList()
{
	DeviceDisplayItems.Empty();

	if (!KnownDevices.IsEmpty())
	{
		// Group known devices by platform user.
		TMap<FPlatformUserId, TArray<FInputDeviceId>> UserDevices;
		for (const TPair<FInputDeviceId, TSharedPtr<FKnownDevice>>& Pair : KnownDevices)
		{
			UserDevices.FindOrAdd(Pair.Value->UserId).Add(Pair.Key);
		}

		// Sort users: the PC's platform user first, then numerically by ID.
		const FPlatformUserId PCUserId = WeakPC.IsValid() ? WeakPC->GetPlatformUserId() : FPlatformUserId();
		const bool bHasPCUser = WeakPC.IsValid();

		TArray<FPlatformUserId> SortedUsers;
		UserDevices.GetKeys(SortedUsers);
		SortedUsers.Sort([&PCUserId, bHasPCUser](const FPlatformUserId& A, const FPlatformUserId& B)
		{
			if (bHasPCUser)
			{
				const bool bAIsPC = (A == PCUserId);
				const bool bBIsPC = (B == PCUserId);
				if (bAIsPC != bBIsPC)
				{
					return bAIsPC;
				}
			}
			return A.GetInternalId() < B.GetInternalId();
		});

		for (const FPlatformUserId& UserId : SortedUsers)
		{
			const TArray<FInputDeviceId>& Devices = UserDevices[UserId];

			const bool bAnyConnected = Devices.ContainsByPredicate([this](const FInputDeviceId& Id)
			{
				return KnownDevices[Id]->LastState == EInputDeviceConnectionState::Connected;
			});

			// User group header item.
			TSharedPtr<FDeviceListItem> HeaderItem = MakeShared<FDeviceListItem>();
			HeaderItem->bIsUserHeader = true;
			HeaderItem->UserId = UserId;
			HeaderItem->bAnyConnected = bAnyConnected;
			HeaderItem->DeviceCount = Devices.Num();
			DeviceDisplayItems.Add(MoveTemp(HeaderItem));

			// One item per device under this user.
			for (const FInputDeviceId& DeviceId : Devices)
			{
				TSharedPtr<FDeviceListItem> DevItem = MakeShared<FDeviceListItem>();
				DevItem->bIsUserHeader = false;
				DevItem->UserId = UserId;
				DevItem->DeviceId = DeviceId;
				DeviceDisplayItems.Add(MoveTemp(DevItem));
			}
		}
	}

	if (DeviceListView.IsValid())
	{
		DeviceListView->RequestListRefresh();

		// Restore the visual selection after the list rebuild.
		if (SelectedDeviceId.IsSet())
		{
			for (const TSharedPtr<FDeviceListItem>& Item : DeviceDisplayItems)
			{
				if (!Item->bIsUserHeader && Item->DeviceId == SelectedDeviceId.GetValue())
				{
					DeviceListView->SetSelection(Item, ESelectInfo::Direct);
					break;
				}
			}
		}
	}
}

TSharedRef<ITableRow> SInputDevicesTab::OnGenerateDeviceRow(
	TSharedPtr<FDeviceListItem> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (Item->bIsUserHeader)
	{
		return SNew(STableRow<TSharedPtr<FDeviceListItem>>, OwnerTable)
			.ShowSelection(false)
			.Padding(FMargin(4.f, 6.f, 4.f, 2.f))
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(6.f, 3.f))
				[
					SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("UserHeader", "Platform User {0}  ({1} device(s))"),
						FText::AsNumber(Item->UserId.GetInternalId()),
						FText::AsNumber(Item->DeviceCount)))
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.ColorAndOpacity(Item->bAnyConnected
						? FLinearColor(0.2f, 0.9f, 0.2f)
						: FLinearColor(0.9f, 0.5f, 0.1f))
				]
			];
	}

	const TSharedPtr<FKnownDevice>* DevicePtr = KnownDevices.Find(Item->DeviceId);
	if (!DevicePtr)
	{
		return SNew(STableRow<TSharedPtr<FDeviceListItem>>, OwnerTable);
	}

	return SNew(SDeviceTableRow, OwnerTable)
		.Item(Item)
		.Device(**DevicePtr)
		.bIsSimulated(ActiveSimulatedDescriptors.Contains(Item->DeviceId));
}

void SInputDevicesTab::OnDeviceConnectionChanged(EInputDeviceConnectionState NewState, FPlatformUserId UserId, FInputDeviceId DeviceId)
{
	if (TSharedPtr<FKnownDevice>* Existing = KnownDevices.Find(DeviceId))
	{
		(*Existing)->LastState = NewState;
	}
	else
	{
		TSharedPtr<FKnownDevice> Entry = MakeShared<FKnownDevice>();
		Entry->UserId = UserId;
		Entry->LastState = NewState;
		if (UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
		{
			Entry->HardwareId = DeviceSubsystem->GetInputDeviceHardwareIdentifier(DeviceId);
		}
		KnownDevices.Add(DeviceId, MoveTemp(Entry));
	}

	RebuildDeviceList();
	if (SelectedDeviceId.IsSet() && SelectedDeviceId.GetValue() == DeviceId)
	{
		RebuildDetails();
	}
}

void SInputDevicesTab::OnHardwareDeviceChanged(FPlatformUserId UserId, FInputDeviceId DeviceId)
{
	// This delegate fires every tick while input is being processed, so bail out early
	// if the hardware identifier hasn't actually changed to avoid thrashing the list view.
	if (TSharedPtr<FKnownDevice>* Existing = KnownDevices.Find(DeviceId))
	{
		if (UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
		{
			const FHardwareDeviceIdentifier NewId = DeviceSubsystem->GetInputDeviceHardwareIdentifier(DeviceId);
			if (NewId == (*Existing)->HardwareId)
			{
				return;
			}
			(*Existing)->HardwareId = NewId;
		}
		RebuildDeviceList();
		if (SelectedDeviceId.IsSet() && SelectedDeviceId.GetValue() == DeviceId)
		{
			RebuildDetails();
		}
	}
}

void SInputDevicesTab::OnDevicePairingChanged(FInputDeviceId DeviceId, FPlatformUserId NewUserId, FPlatformUserId OldUserId)
{
	if (TSharedPtr<FKnownDevice>* Existing = KnownDevices.Find(DeviceId))
	{
		(*Existing)->UserId = NewUserId;
	}
	else
	{
		TSharedPtr<FKnownDevice> Entry = MakeShared<FKnownDevice>();
		Entry->UserId = NewUserId;
		Entry->LastState = IPlatformInputDeviceMapper::Get().GetInputDeviceConnectionState(DeviceId);
		if (UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get())
		{
			Entry->HardwareId = DeviceSubsystem->GetInputDeviceHardwareIdentifier(DeviceId);
		}
		KnownDevices.Add(DeviceId, MoveTemp(Entry));
	}

	RebuildDeviceList();
	if (SelectedDeviceId.IsSet() && SelectedDeviceId.GetValue() == DeviceId)
	{
		RebuildDetails();
	}
}

void SInputDevicesTab::RebuildDetails()
{
	if (!DetailsBox.IsValid())
	{
		return;
	}

	if (!SelectedDeviceId.IsSet())
	{
		DetailsBox->SetContent(
			SNew(SBox).Padding(FMargin(8.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoSelection", "Select a device to view details."))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
			]
		);
		return;
	}

	const TSharedPtr<FKnownDevice>* DevicePtr = KnownDevices.Find(SelectedDeviceId.GetValue());
	if (!DevicePtr)
	{
		SelectedDeviceId.Reset();
		RebuildDetails();
		return;
	}

	const FKnownDevice& Device = **DevicePtr;
	const FInputDeviceId DeviceId = SelectedDeviceId.GetValue();

	// ── Left panel: device info ───────────────────────────────────────────────
	TSharedRef<SScrollBox> InfoBox = SNew(SScrollBox);

	auto AddRow = [&InfoBox](FText Label, TSharedRef<SWidget> Value)
	{
		InfoBox->AddSlot().Padding(FMargin(8.f, 3.f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				.MinDesiredWidth(160.f)
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[ Value ]
		];
	};

	auto AddTextRow = [&AddRow](FText Label, FText Value, FLinearColor Color = FLinearColor(0.9f, 0.9f, 0.9f))
	{
		AddRow(MoveTemp(Label), SNew(STextBlock).Text(MoveTemp(Value)).ColorAndOpacity(Color));
	};

	const EInputDeviceConnectionState State = Device.LastState;

	AddTextRow(LOCTEXT("DetailDeviceId",     "Device ID:"),       FText::AsNumber(DeviceId.GetId()));
	AddTextRow(LOCTEXT("DetailPlatformUser", "Platform User:"),   FText::AsNumber(Device.UserId.GetInternalId()));
	AddTextRow(LOCTEXT("DetailState",        "Connection State:"),
		GetConnectionStateText(State), GetConnectionStateColor(State).GetSpecifiedColor());
	AddTextRow(LOCTEXT("DetailDeviceType",   "Device Type:"),
		GetDeviceTypeText(Device.HardwareId), FLinearColor(0.7f, 0.7f, 0.7f));
	AddTextRow(LOCTEXT("DetailInputClass",   "Input Class:"),
		Device.HardwareId.InputClassName.IsNone()
			? LOCTEXT("DetailUnknownInputClass", "(unknown)") : FText::FromName(Device.HardwareId.InputClassName),
		FLinearColor(0.7f, 0.7f, 0.7f));
	AddTextRow(LOCTEXT("DetailHardwareId",   "Hardware Identifier:"),
		Device.HardwareId.HardwareDeviceIdentifier.IsNone()
			? LOCTEXT("DetailUnknownHardware", "(unknown)") : FText::FromName(Device.HardwareId.HardwareDeviceIdentifier),
		FLinearColor(0.7f, 0.7f, 0.7f));
	AddTextRow(LOCTEXT("DetailFeatures",     "Supported Features:"),
		BuildSupportedFeaturesText(Device.HardwareId.SupportedFeaturesMask), FLinearColor(0.7f, 0.7f, 0.9f));

	// ── Right panel: simulated descriptor controls ────────────────────────────
	const FHardwareDeviceIdentifier* ActiveSim = ActiveSimulatedDescriptors.Find(DeviceId);

	TSharedRef<SVerticalBox> SimPanel = SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SimDescHeader", "Simulated Descriptor"))
			.Font(FAppStyle::GetFontStyle("BoldFont"))
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 8.f)
		[
			SNew(STextBlock)
			.Text_Lambda([this, DeviceId]() -> FText
			{
				const FHardwareDeviceIdentifier* Sim = ActiveSimulatedDescriptors.Find(DeviceId);
				if (!Sim) { return LOCTEXT("SimNone", "(none)"); }
				return FText::Format(LOCTEXT("SimActiveFmt", "{0}  ({1})"),
					FText::FromName(Sim->HardwareDeviceIdentifier),
					FText::FromName(Sim->InputClassName));
			})
			.ColorAndOpacity_Lambda([this, DeviceId]() -> FSlateColor
			{
				return ActiveSimulatedDescriptors.Contains(DeviceId)
					? FLinearColor(0.2f, 0.9f, 0.9f) : FLinearColor(0.5f, 0.5f, 0.5f);
			})
			.AutoWrapText(true)
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			SNew(SComboBox<TSharedPtr<FHardwareDeviceIdentifier>>)
			.OptionsSource(&AvailableHardwareDevices)
			.InitiallySelectedItem(PendingSimDescriptors.FindOrAdd(DeviceId,
				AvailableHardwareDevices.IsEmpty() ? nullptr : AvailableHardwareDevices[0]))
			.OnSelectionChanged_Lambda([this, DeviceId](TSharedPtr<FHardwareDeviceIdentifier> NewItem, ESelectInfo::Type)
			{
				PendingSimDescriptors.Add(DeviceId, NewItem);
			})
			.OnGenerateWidget_Lambda([](TSharedPtr<FHardwareDeviceIdentifier> Opt) -> TSharedRef<SWidget>
			{
				if (!Opt.IsValid()) { return SNew(STextBlock).Text(FText::GetEmpty()); }
				return SNew(STextBlock)
					.Text(FText::Format(LOCTEXT("SimItemFmt", "{0}  ({1})"),
						FText::FromName(Opt->HardwareDeviceIdentifier),
						FText::FromName(Opt->InputClassName)))
					.Margin(FMargin(4.f, 2.f));
			})
			.Visibility(AvailableHardwareDevices.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			[
				SNew(STextBlock)
				.Text_Lambda([this, DeviceId]() -> FText
				{
					const TSharedPtr<FHardwareDeviceIdentifier>* Pending = PendingSimDescriptors.Find(DeviceId);
					if (!Pending || !Pending->IsValid()) { return LOCTEXT("NoneSelected", "(none)"); }
					return FText::Format(LOCTEXT("SimItemFmt", "{0}  ({1})"),
						FText::FromName((*Pending)->HardwareDeviceIdentifier),
						FText::FromName((*Pending)->InputClassName));
				})
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0.f, 0.f, 0.f, 4.f)
		[
			SNew(SButton)
			.Text(LOCTEXT("SetSimBtn", "Set Simulated"))
			.HAlign(HAlign_Center)
			.IsEnabled_Lambda([this, DeviceId]() -> bool
			{
				const TSharedPtr<FHardwareDeviceIdentifier>* Pending = PendingSimDescriptors.Find(DeviceId);
				return Pending && Pending->IsValid();
			})
			.Visibility(AvailableHardwareDevices.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			.OnClicked_Lambda([this, DeviceId]() -> FReply
			{
				const TSharedPtr<FHardwareDeviceIdentifier>* Pending = PendingSimDescriptors.Find(DeviceId);
				if (Pending && Pending->IsValid())
				{
#if !UE_BUILD_SHIPPING
					FInputDeviceDescriptor Desc;
					Desc.HardwareDeviceHandle     = DeviceId;
					Desc.InputDeviceName          = (*Pending)->InputClassName;
					Desc.HardwareDeviceIdentifier = (*Pending)->HardwareDeviceIdentifier;
					FInputDeviceRegistry::SetSimulatedDescriptor(DeviceId, Desc);
#endif
					ActiveSimulatedDescriptors.Add(DeviceId, **Pending);
					RebuildDeviceList();
					RebuildDetails();
				}
				return FReply::Handled();
			})
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SButton)
			.Text(LOCTEXT("ClearSimBtn", "Clear Simulated"))
			.HAlign(HAlign_Center)
			.IsEnabled_Lambda([this, DeviceId]() -> bool
			{
				return ActiveSimulatedDescriptors.Contains(DeviceId);
			})
			.Visibility(AvailableHardwareDevices.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible)
			.OnClicked_Lambda([this, DeviceId]() -> FReply
			{
#if !UE_BUILD_SHIPPING
				FInputDeviceRegistry::ClearSimulatedDescriptor(DeviceId);
#endif
				ActiveSimulatedDescriptors.Remove(DeviceId);
				PendingSimDescriptors.Remove(DeviceId);
				RebuildDeviceList();
				RebuildDetails();
				return FReply::Handled();
			})
		];

	// ── Assemble: info left, sim controls right ───────────────────────────────
	DetailsBox->SetContent(
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.f)
		[ InfoBox ]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(12.f, 6.f, 8.f, 6.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(FMargin(10.f, 8.f))
			[ SimPanel ]
		]
	);
}

#undef LOCTEXT_NAMESPACE