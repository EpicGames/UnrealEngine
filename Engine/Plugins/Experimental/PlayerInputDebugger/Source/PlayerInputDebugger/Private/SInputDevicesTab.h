// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformInputDeviceMapper.h"
#include "GameFramework/InputSettings.h"
#include "GameFramework/PlayerController.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"

// Cached record for a device that has been seen at least once.
// Retained even after disconnection so it stays visible in the list.
struct FKnownDevice
{
	FPlatformUserId UserId;
	FHardwareDeviceIdentifier HardwareId;
	EInputDeviceConnectionState LastState = EInputDeviceConnectionState::Unknown;
};

// A single item in the device list view — either a platform-user group header or a device row.
struct FDeviceListItem
{
	bool bIsUserHeader = false;
	FPlatformUserId UserId;
	FInputDeviceId DeviceId;
	bool bAnyConnected = false;  // header tint: green if any device still connected
	int32 DeviceCount = 0;       // only meaningful for header items
};

class SBox;

// Tab that shows input hardware devices per platform user.
// Devices remain visible after disconnecting, shown with a "Disconnected" state.
class SInputDevicesTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInputDevicesTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SInputDevicesTab();

	void SetPlayerController(APlayerController* PC) { WeakPC = PC; RebuildDeviceList(); }

private:
	void RebuildDeviceList();
	void RebuildDetails();
	TSharedRef<ITableRow> OnGenerateDeviceRow(TSharedPtr<FDeviceListItem> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void OnDeviceConnectionChanged(EInputDeviceConnectionState NewState, FPlatformUserId UserId, FInputDeviceId DeviceId);
	void OnHardwareDeviceChanged(FPlatformUserId UserId, FInputDeviceId DeviceId);
	void OnDevicePairingChanged(FInputDeviceId DeviceId, FPlatformUserId NewUserId, FPlatformUserId OldUserId);

	TArray<TSharedPtr<FDeviceListItem>> DeviceDisplayItems;
	TSharedPtr<SListView<TSharedPtr<FDeviceListItem>>> DeviceListView;
	TSharedPtr<SBox> DetailsBox;
	TOptional<FInputDeviceId> SelectedDeviceId;
	FDelegateHandle ConnectionChangedHandle;
	FDelegateHandle HardwareDeviceChangedHandle;
	FDelegateHandle PairingChangeHandle;
	TMap<FInputDeviceId, TSharedPtr<FKnownDevice>> KnownDevices;
	TWeakObjectPtr<APlayerController> WeakPC;

	// Simulated descriptor support
	TArray<TSharedPtr<FHardwareDeviceIdentifier>> AvailableHardwareDevices;
	TMap<FInputDeviceId, FHardwareDeviceIdentifier> ActiveSimulatedDescriptors;
	// Per-device pending selection in the inline sim descriptor combo box.
	TMap<FInputDeviceId, TSharedPtr<FHardwareDeviceIdentifier>> PendingSimDescriptors;
};
