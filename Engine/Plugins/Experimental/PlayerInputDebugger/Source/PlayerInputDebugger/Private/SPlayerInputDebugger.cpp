// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlayerInputDebugger.h"

#include "SEnhancedInputTab.h"
#include "SInputDevicesTab.h"
#include "SPlayerInputEventsTab.h"
#include "SGlobalInputInfoSection.h"

#include "Editor.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/LayoutService.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "GameFramework/GameModeBase.h"
#include "GameFramework/PlayerController.h"
#include "Styling/AppStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SPlayerInputDebugger"

namespace PlayerInputDebuggerTabs
{
	static const FName InputEvents   ("PlayerInputDebugger.InputEvents");
	static const FName InputDevices  ("PlayerInputDebugger.InputDevices");
	static const FName EnhancedInput ("PlayerInputDebugger.EnhancedInput");
}

void SPlayerInputDebugger::Construct(const FArguments& InArgs)
{
	// Build the three tab content widgets up front so we can pass them into
	// the spawner lambdas and also call SetPlayerController on PC changes.
	SAssignNew(PlayerInputEventsTab, SPlayerInputEventsTab);
	SAssignNew(DevicesTab,           SInputDevicesTab);
	SAssignNew(EnhancedInputTab,     SEnhancedInputTab);
	SAssignNew(GlobalInfoSection,    SGlobalInputInfoSection);

	// -----------------------------------------------------------------------
	// Local tab manager — parented to the host nomad tab so sub-tabs are
	// scoped to this window and can be dragged/docked within it.
	// -----------------------------------------------------------------------
	TSharedPtr<SDockTab> ParentTab = InArgs._ParentTab;
	check(ParentTab.IsValid());

	TabManager = FGlobalTabmanager::Get()->NewTabManager(ParentTab.ToSharedRef());

	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateStatic(
		[](const TSharedRef<FTabManager::FLayout>& InLayout)
		{
			if (InLayout->GetPrimaryArea().Pin().IsValid())
			{
				FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
			}
		}));

	// Save layout and tear down areas when the host nomad tab is closed.
	ParentTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(
		this, &SPlayerInputDebugger::OnParentTabClosed));

	// Register a spawner for each sub-tab.  The lambdas capture a TSharedRef to
	// the already-constructed content widget so the same instance is reused if
	// the tab is closed and re-opened via the Window menu.
	{
		TSharedRef<SPlayerInputEventsTab> Content = PlayerInputEventsTab.ToSharedRef();
		TabManager->RegisterTabSpawner(PlayerInputDebuggerTabs::InputEvents,
			FOnSpawnTab::CreateLambda([Content](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("TabInputEvents", "Input Events"))
					[ Content ];
			}))
			.SetDisplayName(LOCTEXT("TabInputEvents", "Input Events"));
	}
	{
		TSharedRef<SInputDevicesTab> Content = DevicesTab.ToSharedRef();
		TabManager->RegisterTabSpawner(PlayerInputDebuggerTabs::InputDevices,
			FOnSpawnTab::CreateLambda([Content](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("TabInputDevices", "Input Devices"))
					[ Content ];
			}))
			.SetDisplayName(LOCTEXT("TabInputDevices", "Input Devices"));
	}
	{
		TSharedRef<SEnhancedInputTab> Content = EnhancedInputTab.ToSharedRef();
		TabManager->RegisterTabSpawner(PlayerInputDebuggerTabs::EnhancedInput,
			FOnSpawnTab::CreateLambda([Content](const FSpawnTabArgs&) -> TSharedRef<SDockTab>
			{
				return SNew(SDockTab)
					.TabRole(ETabRole::PanelTab)
					.Label(LOCTEXT("TabEnhancedInput", "Enhanced Input"))
					[ Content ];
			}))
			.SetDisplayName(LOCTEXT("TabEnhancedInput", "Enhanced Input"));
	}

	// Default layout: all three tabs in a single stack, Enhanced Input foregrounded.
	// The user can drag tabs out into separate areas; the layout is persisted to
	// GEditorLayoutIni and restored on next open.
	TabLayout = FTabManager::NewLayout("Standalone_PlayerInputDebugger_Layout_v1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
				->AddTab(PlayerInputDebuggerTabs::InputEvents,   ETabState::OpenedTab)
				->AddTab(PlayerInputDebuggerTabs::InputDevices,  ETabState::OpenedTab)
				->AddTab(PlayerInputDebuggerTabs::EnhancedInput, ETabState::OpenedTab)
				->SetForegroundTab(PlayerInputDebuggerTabs::EnhancedInput)
			)
		);

	TabLayout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, TabLayout.ToSharedRef());

	TSharedRef<SWidget> TabContent = TabManager->RestoreFrom(
		TabLayout.ToSharedRef(), TSharedPtr<SWindow>()).ToSharedRef();

	// -----------------------------------------------------------------------
	// Window menu — lets the user re-open sub-tabs they've closed.
	// -----------------------------------------------------------------------
	TSharedPtr<FUICommandList> CommandList = MakeShared<FUICommandList>();
	TWeakPtr<FTabManager> WeakTabManager = TabManager;

	const auto ToggleTab = [](TWeakPtr<FTabManager> InMgr, FName InTab)
	{
		if (TSharedPtr<FTabManager> Mgr = InMgr.Pin())
		{
			if (TSharedPtr<SDockTab> Existing = Mgr->FindExistingLiveTab(InTab))
			{
				Existing->RequestCloseTab();
			}
			else
			{
				Mgr->TryInvokeTab(InTab);
			}
		}
	};
	const auto IsTabVisible = [](TWeakPtr<FTabManager> InMgr, FName InTab) -> bool
	{
		if (TSharedPtr<FTabManager> Mgr = InMgr.Pin())
		{
			return Mgr->FindExistingLiveTab(InTab).IsValid();
		}
		return false;
	};

	FMenuBarBuilder MenuBarBuilder(CommandList);
	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateLambda([WeakTabManager, ToggleTab, IsTabVisible](FMenuBuilder& Builder)
		{
			Builder.AddMenuEntry(
				LOCTEXT("TabInputEvents",  "Input Events"),
				FText::GetEmpty(), FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakTabManager, ToggleTab]()  { ToggleTab(WeakTabManager, PlayerInputDebuggerTabs::InputEvents); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([WeakTabManager, IsTabVisible]() { return IsTabVisible(WeakTabManager, PlayerInputDebuggerTabs::InputEvents); })
				), NAME_None, EUserInterfaceActionType::ToggleButton);

			Builder.AddMenuEntry(
				LOCTEXT("TabInputDevices", "Input Devices"),
				FText::GetEmpty(), FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakTabManager, ToggleTab]()  { ToggleTab(WeakTabManager, PlayerInputDebuggerTabs::InputDevices); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([WeakTabManager, IsTabVisible]() { return IsTabVisible(WeakTabManager, PlayerInputDebuggerTabs::InputDevices); })
				), NAME_None, EUserInterfaceActionType::ToggleButton);

			Builder.AddMenuEntry(
				LOCTEXT("TabEnhancedInput", "Enhanced Input"),
				FText::GetEmpty(), FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([WeakTabManager, ToggleTab]()  { ToggleTab(WeakTabManager, PlayerInputDebuggerTabs::EnhancedInput); }),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([WeakTabManager, IsTabVisible]() { return IsTabVisible(WeakTabManager, PlayerInputDebuggerTabs::EnhancedInput); })
				), NAME_None, EUserInterfaceActionType::ToggleButton);
		})
	);

	TSharedRef<SWidget> MenuBarWidget = MenuBarBuilder.MakeWidget();
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuBarWidget);

	// -----------------------------------------------------------------------
	// Root layout: menu bar / PC selector / global info / tab area
	// -----------------------------------------------------------------------
	ChildSlot
	[
		SNew(SVerticalBox)

		// Window menu bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			MenuBarWidget
		]

		// Player controller selector
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(6.f, 4.f, 6.f, 4.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 6.f, 0.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PCLabel", "Player Controller:"))
			]

			+ SHorizontalBox::Slot()
			.MaxWidth(320.f)
			.AutoWidth()
			[
				SAssignNew(PCCombo, SComboBox<TSharedPtr<FPlayerControllerEntry>>)
				.OptionsSource(&PCEntries)
				.OnGenerateWidget(this, &SPlayerInputDebugger::MakePCComboEntry)
				.OnSelectionChanged(this, &SPlayerInputDebugger::OnPCSelected)
				.Content()
				[
					SNew(STextBlock)
					.Text(this, &SPlayerInputDebugger::GetSelectedPCText)
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(4.f, 0.f, 0.f, 0.f)
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("RefreshTooltip", "Refresh player controller list"))
				.OnClicked_Lambda([this]() -> FReply
				{
					RefreshPlayerControllers();
					return FReply::Handled();
				})
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Refresh"))
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
				]
			]
		]

		// Global input overview — always visible above the tab area
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			GlobalInfoSection.ToSharedRef()
		]

		// Dockable tab area
		+ SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.DarkGroupBorder"))
			.Padding(FMargin(0.f, 2.f))
			[
				TabContent
			]
		]
	];

	// BeginPIE clears stale entries; GameModePostLoginEvent repopulates as PCs are created.
	BeginPIEHandle  = FEditorDelegates::BeginPIE.AddRaw(this, &SPlayerInputDebugger::OnBeginPIE);
	EndPIEHandle    = FEditorDelegates::EndPIE.AddRaw(this, &SPlayerInputDebugger::OnEndPIE);
	PostLoginHandle = FGameModeEvents::OnGameModePostLoginEvent().AddRaw(this, &SPlayerInputDebugger::OnGameModePostLogin);
	LogoutHandle    = FGameModeEvents::OnGameModeLogoutEvent().AddRaw(this, &SPlayerInputDebugger::OnGameModeLogout);

	// Populate if already in PIE.
	RefreshPlayerControllers();
}

SPlayerInputDebugger::~SPlayerInputDebugger()
{
	if (TabManager.IsValid())
	{
		TabManager->UnregisterAllTabSpawners();
		TabManager.Reset();
	}

	FEditorDelegates::BeginPIE.Remove(BeginPIEHandle);
	FEditorDelegates::EndPIE.Remove(EndPIEHandle);
	FGameModeEvents::OnGameModePostLoginEvent().Remove(PostLoginHandle);
	FGameModeEvents::OnGameModeLogoutEvent().Remove(LogoutHandle);
}

void SPlayerInputDebugger::OnParentTabClosed(TSharedRef<SDockTab> /*ClosedTab*/)
{
	if (TabManager.IsValid())
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, TabManager->PersistLayout());
		TabManager->CloseAllAreas();
	}
}

void SPlayerInputDebugger::RefreshPlayerControllers(const bool bIsPlayEnding /*= false*/)
{
	PCEntries.Empty();

	if (!GEngine)
	{
		return;
	}

	if (!bIsPlayEnding)
	{
		int32 PCIndex = 0;
		for (const FWorldContext& Context : GEngine->GetWorldContexts())
		{
			if (Context.WorldType != EWorldType::PIE)
			{
				continue;
			}

			UWorld* World = Context.World();
			if (!World)
			{
				continue;
			}

			for (FConstPlayerControllerIterator It = World->GetPlayerControllerIterator(); It; ++It)
			{
				if (APlayerController* PC = It->Get())
				{
					TSharedPtr<FPlayerControllerEntry> Entry = MakeShared<FPlayerControllerEntry>();
					Entry->PC = PC;
					Entry->DisplayName = FText::Format(
						LOCTEXT("PCEntryFmt", "Player Controller {0} ({1})"),
						FText::AsNumber(PCIndex),
						FText::FromString(PC->GetName()));
					PCEntries.Add(MoveTemp(Entry));
					++PCIndex;
				}
			}
		}
	}

	if (PCCombo.IsValid())
	{
		PCCombo->RefreshOptions();
	}

	// Auto-select the first entry.
	if (!PCEntries.IsEmpty())
	{
		SelectEntry(PCEntries[0]);
	}
	else
	{
		SelectEntry(nullptr);
	}
}

void SPlayerInputDebugger::OnBeginPIE(const bool bIsSimulating)
{
	// Clear stale entries from any previous session.
	// The list is repopulated by OnGameModePostLogin as player controllers are created.
	PCEntries.Empty();
	if (PCCombo.IsValid())
	{
		PCCombo->RefreshOptions();
	}
	SelectEntry(nullptr);
}

void SPlayerInputDebugger::OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer)
{
	RefreshPlayerControllers();
}

void SPlayerInputDebugger::OnGameModeLogout(AGameModeBase* GameMode, AController* Exiting)
{
	// Guard against world teardown: GameModeLogoutEvent fires for each PC as the PIE
	// world is destroyed, which can occur after EndPIE has already cleared the list.
	// Refreshing during teardown would re-populate the dropdown from the dying world.
	if (GEditor && GEditor->IsPlayingSessionInEditor())
	{
		//RefreshPlayerControllers();
	}
}

void SPlayerInputDebugger::OnEndPIE(const bool bIsSimulating)
{
	PCEntries.Empty();
	if (PCCombo.IsValid())
	{
		PCCombo->RefreshOptions();
	}
	SelectEntry(nullptr);

	RefreshPlayerControllers(true);
}

void SPlayerInputDebugger::OnPCSelected(TSharedPtr<FPlayerControllerEntry> Entry, ESelectInfo::Type SelectInfo)
{
	SelectEntry(Entry);
}

void SPlayerInputDebugger::SelectEntry(TSharedPtr<FPlayerControllerEntry> Entry)
{
	SelectedEntry = Entry;
	APlayerController* PC = Entry.IsValid() ? Entry->PC.Get() : nullptr;

	if (EnhancedInputTab.IsValid())
	{
		EnhancedInputTab->SetPlayerController(PC);
	}

	if (DevicesTab.IsValid())
	{
		DevicesTab->SetPlayerController(PC);
	}

	if (PlayerInputEventsTab.IsValid())
	{
		PlayerInputEventsTab->SetPlayerController(PC);
	}

	if (GlobalInfoSection.IsValid())
	{
		GlobalInfoSection->SetPlayerController(PC);
	}
}

TSharedRef<SWidget> SPlayerInputDebugger::MakePCComboEntry(TSharedPtr<FPlayerControllerEntry> Entry)
{
	return SNew(STextBlock)
		.Text(Entry.IsValid() ? Entry->DisplayName : LOCTEXT("NullPC", "(none)"))
		.Margin(FMargin(4.f, 2.f));
}

FText SPlayerInputDebugger::GetSelectedPCText() const
{
	if (SelectedEntry.IsValid() && SelectedEntry->PC.IsValid())
	{
		return SelectedEntry->DisplayName;
	}
	return LOCTEXT("NoPCSelected", "(No player controller — start PIE)");
}

#undef LOCTEXT_NAMESPACE
