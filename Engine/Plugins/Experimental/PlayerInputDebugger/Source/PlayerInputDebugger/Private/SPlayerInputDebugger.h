// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/PlayerController.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Framework/Docking/TabManager.h"

class AController;
class AGameModeBase;
class FTabManager;
class SDockTab;
class SInputDevicesTab;
class SEnhancedInputTab;
class SPlayerInputEventsTab;
class SGlobalInputInfoSection;

// Entry in the player controller dropdown.
struct FPlayerControllerEntry
{
	TWeakObjectPtr<APlayerController> PC;
	FText DisplayName;
};

// Root widget for the Player Input Debugger window.
// Contains a player controller selector, a persistent global info section,
// and three dockable sub-tabs managed by a local FTabManager.
class SPlayerInputDebugger : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlayerInputDebugger) {}
		// The nomad SDockTab that hosts this widget. Required to create the local tab manager.
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SPlayerInputDebugger();

private:
	void RefreshPlayerControllers(const bool bIsPlayEnding = false);
	void OnBeginPIE(const bool bIsSimulating);
	void OnEndPIE(const bool bIsSimulating);
	void OnGameModePostLogin(AGameModeBase* GameMode, APlayerController* NewPlayer);
	void OnGameModeLogout(AGameModeBase* GameMode, AController* Exiting);
	void OnPCSelected(TSharedPtr<FPlayerControllerEntry> Entry, ESelectInfo::Type SelectInfo);
	TSharedRef<SWidget> MakePCComboEntry(TSharedPtr<FPlayerControllerEntry> Entry);
	FText GetSelectedPCText() const;
	void SelectEntry(TSharedPtr<FPlayerControllerEntry> Entry);
	void OnParentTabClosed(TSharedRef<SDockTab> ClosedTab);

	TArray<TSharedPtr<FPlayerControllerEntry>> PCEntries;
	TSharedPtr<FPlayerControllerEntry> SelectedEntry;

	TSharedPtr<SComboBox<TSharedPtr<FPlayerControllerEntry>>> PCCombo;

	TSharedPtr<SPlayerInputEventsTab> PlayerInputEventsTab;
	TSharedPtr<SInputDevicesTab> DevicesTab;
	TSharedPtr<SEnhancedInputTab> EnhancedInputTab;
	TSharedPtr<SGlobalInputInfoSection> GlobalInfoSection;

	TSharedPtr<FTabManager> TabManager;
	TSharedPtr<FTabManager::FLayout> TabLayout;

	FDelegateHandle BeginPIEHandle;
	FDelegateHandle EndPIEHandle;
	FDelegateHandle PostLoginHandle;
	FDelegateHandle LogoutHandle;
};
