// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlayerInputEventsTab.h"

#include "Blueprint/UserWidget.h"
#include "Brushes/SlateColorBrush.h"
#include "Components/InputComponent.h"
#include "Debugging/SlateDebugging.h"
#include "Editor.h"
#include "Engine/Engine.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "Input/Events.h"
#include "InputAction.h"
#include "Slate/SObjectWidget.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateTypes.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "UObject/Stack.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

#define LOCTEXT_NAMESPACE "SPlayerInputEventsTab"

namespace InputEventsTab
{
	static const float FrameColumnWidth   = 70.f;
	static const float SourceColumnWidth  = 80.f;
	static const float EventColumnWidth   = 190.f;
	static const float KeyColumnWidth     = 150.f;
	static const float PCColumnWidth      = 120.f;
	static const float HandlerColumnWidth = 210.f;
}

// ── Helpers ───────────────────────────────────────────────────────────────────

// Slate row style: dark blue tint to distinguish from Player rows.
static const FTableRowStyle& GetSlateRowStyle()
{
	static const FTableRowStyle Style = FTableRowStyle(FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
		.SetEvenRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.05f, 0.05f, 0.13f)))
		.SetOddRowBackgroundBrush(FSlateColorBrush(FLinearColor(0.04f, 0.04f, 0.10f)));
	return Style;
}

static FText GetSlateEventTypeText(ESlateDebuggingInputEvent Type)
{
	if (const UEnum* E = StaticEnum<ESlateDebuggingInputEvent>())
	{
		return E->GetDisplayNameTextByValue((int64)Type);
	}
	return FText::AsNumber((int32)Type);
}

// Compact value display: omit trailing zero axes.
static FString FormatInputValue(const FVector& V)
{
	if (V.Z != 0.f) { return FString::Printf(TEXT("%.3f, %.3f, %.3f"), V.X, V.Y, V.Z); }
	if (V.Y != 0.f) { return FString::Printf(TEXT("%.3f, %.3f"), V.X, V.Y); }
	return FString::Printf(TEXT("%.3f"), V.X);
}

// ── Row widget ────────────────────────────────────────────────────────────────

namespace
{

class SUnifiedInputEventRow : public SMultiColumnTableRow<TSharedPtr<FUnifiedInputEventRecord>>
{
public:
	SLATE_BEGIN_ARGS(SUnifiedInputEventRow) {}
		SLATE_ARGUMENT(TSharedPtr<FUnifiedInputEventRecord>, Item)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
	{
		Item = InArgs._Item;
		FSuperRowType::FArguments SuperArgs = FSuperRowType::FArguments().Padding(FMargin(2.f, 1.f));
		if (Item->Source == EInputEventSource::Slate || Item->Source == EInputEventSource::SlateMouseCapture)
		{
			SuperArgs.Style(&GetSlateRowStyle());
		}
		SMultiColumnTableRow<TSharedPtr<FUnifiedInputEventRecord>>::Construct(SuperArgs, InOwnerTableView);
	}

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
	{
		check(Item.IsValid());

		const bool bIsFlush        = (Item->Source == EInputEventSource::PlayerFlush);
		const bool bIsMouseCapture = (Item->Source == EInputEventSource::SlateMouseCapture);
		static const FLinearColor FlushColor(0.9f, 0.2f, 0.2f);
		static const FLinearColor DimColor(0.5f, 0.5f, 0.5f);

		// ── Frame ────────────────────────────────────────────────────────────
		if (ColumnName == TEXT("Frame"))
		{
			return SNew(STextBlock)
				.Text(FText::AsNumber(Item->FrameNumber))
				.Font(FAppStyle::GetFontStyle(bIsFlush ? "BoldFont" : "NormalFont"))
				.ColorAndOpacity(bIsFlush ? FlushColor : DimColor);
		}

		// ── Source ───────────────────────────────────────────────────────────
		if (ColumnName == TEXT("Source"))
		{
			FText SourceText;
			FLinearColor SourceColor;
			switch (Item->Source)
			{
			case EInputEventSource::Slate:
			case EInputEventSource::SlateMouseCapture:
				SourceText  = LOCTEXT("SourceSlate",  "Slate");
				SourceColor = FLinearColor(0.4f, 0.7f, 1.0f);   // blue
				break;
			case EInputEventSource::Player:
				SourceText  = LOCTEXT("SourcePlayer", "Player");
				SourceColor = FLinearColor(0.6f, 0.9f, 0.6f);   // green
				break;
			case EInputEventSource::PlayerFlush:
				SourceText  = LOCTEXT("SourcePlayer", "Player");
				SourceColor = FlushColor;
				break;
			}
			return SNew(STextBlock)
				.Text(SourceText)
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(SourceColor);
		}

		// ── Event / Action ───────────────────────────────────────────────────
		if (ColumnName == TEXT("Event"))
		{
			if (bIsFlush)
			{
				return SNew(STextBlock)
					.Text(LOCTEXT("FlushLabel", "--- Input Flushed ---"))
					.Font(FAppStyle::GetFontStyle("BoldFont"))
					.ColorAndOpacity(FlushColor);
			}

			if (bIsMouseCapture)
			{
				const bool bCaptured = Item->bReplyHandled;
				return SNew(STextBlock)
					.Text(bCaptured ? LOCTEXT("MouseCaptured", "Mouse Captured") : LOCTEXT("MouseCaptureLost", "Mouse Capture Lost"))
					.Font(FAppStyle::GetFontStyle("NormalFont"))
					.ColorAndOpacity(bCaptured ? FLinearColor(0.2f, 0.9f, 0.2f) : FLinearColor(0.9f, 0.6f, 0.2f));
			}

			if (Item->Source == EInputEventSource::Slate)
			{
				return SNew(STextBlock)
					.Text(GetSlateEventTypeText(Item->SlateEventType))
					.Font(FAppStyle::GetFontStyle("NormalFont"));
			}

			// Player — action name; hyperlink if we resolved an asset
			if (Item->ActionAsset.IsValid())
			{
				return SNew(SHyperlink)
					.Text(FText::FromString(Item->ActionName))
					.OnNavigate(FSimpleDelegate::CreateLambda([WeakAsset = Item->ActionAsset]()
					{
						if (UObject* Asset = WeakAsset.Get())
						{
							if (UAssetEditorSubsystem* Sub = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
							{
								Sub->OpenEditorForAsset(Asset);
							}
						}
					}));
			}
			return SNew(STextBlock)
				.Text(FText::FromString(Item->ActionName))
				.Font(FAppStyle::GetFontStyle("NormalFont"));
		}

		// ── Key ──────────────────────────────────────────────────────────────
		if (ColumnName == TEXT("Key"))
		{
			if (bIsFlush || bIsMouseCapture || Item->Source == EInputEventSource::Player)
			{
				return SNew(STextBlock).Text(FText::GetEmpty());
			}
			return SNew(STextBlock)
				.Text(Item->Key.IsValid() ? FText::FromString(Item->Key.ToString()) : FText::GetEmpty())
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f));
		}

		// ── Player Controller ─────────────────────────────────────────────────
		if (ColumnName == TEXT("PC"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->PlayerControllerName))
				.Font(FAppStyle::GetFontStyle("NormalFont"))
				.ColorAndOpacity(FLinearColor(0.7f, 0.7f, 0.7f));
		}

		// ── Handler / Component ──────────────────────────────────────────────
		if (ColumnName == TEXT("Handler"))
		{
			if (bIsFlush)
			{
				return SNew(STextBlock).Text(FText::GetEmpty());
			}

			if (bIsMouseCapture)
			{
				return SNew(STextBlock)
					.Text(Item->HandlerWidgetType.IsEmpty() ? FText::GetEmpty() : FText::FromString(Item->HandlerWidgetType))
					.Font(FAppStyle::GetFontStyle("NormalFont"))
					.ColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.2f));  // yellow
			}

			if (Item->Source == EInputEventSource::Slate)
			{
				if (Item->HandlerWidgetType.IsEmpty())
				{
					return SNew(STextBlock)
						.Text(LOCTEXT("NoHandler", "—"))
						.ColorAndOpacity(FLinearColor(0.4f, 0.4f, 0.4f));
				}

				FString WidgetLabel = Item->HandlerWidgetType;
				if (!Item->UserWidgetName.IsEmpty())
				{
					WidgetLabel = FString::Printf(TEXT("%s (%s)"), *Item->HandlerWidgetType, *Item->UserWidgetName);
				}

				if (Item->UserWidgetBlueprint.IsValid())
				{
					static const FHyperlinkStyle BlueLinkStyle = []()
					{
						FHyperlinkStyle Style = FAppStyle::Get().GetWidgetStyle<FHyperlinkStyle>("Hyperlink");
						Style.SetTextStyle(FTextBlockStyle(Style.TextStyle)
							.SetColorAndOpacity(FSlateColor(FLinearColor(0.2f, 0.6f, 1.0f))));
						return Style;
					}();
					TWeakObjectPtr<UObject> WeakBP = Item->UserWidgetBlueprint;
					return SNew(SHyperlink)
						.Text(FText::FromString(WidgetLabel))
						.Style(&BlueLinkStyle)
						.OnNavigate(FSimpleDelegate::CreateLambda([WeakBP]()
						{
							if (UObject* BP = WeakBP.Get())
							{
								if (UAssetEditorSubsystem* Sub = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
								{
									Sub->OpenEditorForAsset(BP);
								}
							}
						}));
				}

				return SNew(STextBlock)
					.Text(FText::FromString(WidgetLabel))
					.Font(FAppStyle::GetFontStyle("NormalFont"))
					.ColorAndOpacity(FLinearColor(0.9f, 0.9f, 0.2f));  // yellow
			}

			// Player event — input component name
			return SNew(STextBlock)
				.Text(FText::FromString(Item->InputComponentName))
				.Font(FAppStyle::GetFontStyle("NormalFont"));
		}

		return SNew(STextBlock).Text(FText::GetEmpty());
	}

private:
	TSharedPtr<FUnifiedInputEventRecord> Item;
};

} // anonymous namespace

// ── Construct / Destruct ──────────────────────────────────────────────────────

void SPlayerInputEventsTab::Construct(const FArguments& InArgs)
{
	TSharedRef<SHeaderRow> HeaderRow = SNew(SHeaderRow)
		+ SHeaderRow::Column("Frame")
		.DefaultLabel(LOCTEXT("ColFrame", "Frame"))
		.ManualWidth(InputEventsTab::FrameColumnWidth)

		+ SHeaderRow::Column("Source")
		.DefaultLabel(LOCTEXT("ColSource", "Source"))
		.ManualWidth(InputEventsTab::SourceColumnWidth)

		+ SHeaderRow::Column("Event")
		.DefaultLabel(LOCTEXT("ColEvent", "Event / Action"))
		.ManualWidth(InputEventsTab::EventColumnWidth)

		+ SHeaderRow::Column("Key")
		.DefaultLabel(LOCTEXT("ColKey", "Key"))
		.ManualWidth(InputEventsTab::KeyColumnWidth)

		+ SHeaderRow::Column("PC")
		.DefaultLabel(LOCTEXT("ColPC", "Player Controller"))
		.ManualWidth(InputEventsTab::PCColumnWidth)

		+ SHeaderRow::Column("Handler")
		.DefaultLabel(LOCTEXT("ColHandler", "Handler / Component"))
		.ManualWidth(InputEventsTab::HandlerColumnWidth);

	TSharedRef<SVerticalBox> VBox = SNew(SVerticalBox)

		// Toolbar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f, 4.f)
		[
			SNew(SHorizontalBox)

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 8.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("ClearBtn", "Clear"))
				.OnClicked_Lambda([this]() -> FReply
				{
					LogItems.Empty();
					SelectedItem.Reset();
					ListView->ClearSelection();
					RebuildDetails();
					ListView->RequestListRefresh();
					return FReply::Handled();
				})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(0.f, 0.f, 8.f, 0.f)
			[
				SNew(SButton)
				.Text(LOCTEXT("ResumeScrollBtn", "Resume Scroll"))
				.IsEnabled_Lambda([this]() { return SelectedItem.IsValid(); })
				.ToolTipText(LOCTEXT("ResumeScrollTip", "Clear the current selection and resume auto-scrolling to new events"))
				.OnClicked_Lambda([this]() -> FReply
				{
					SelectedItem.Reset();
					ListView->ClearSelection();
					RebuildDetails();
					if (bAutoScroll && !LogItems.IsEmpty())
					{
						ListView->RequestScrollIntoView(LogItems.Last());
					}
					return FReply::Handled();
				})
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 8.f, 0.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bAutoScroll ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bAutoScroll = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("AutoScrollLabel", "Auto-scroll"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.f, 0.f, 8.f, 0.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bIgnoreDebuggerWindowEvents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bIgnoreDebuggerWindowEvents = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("IgnoreDebuggerLabel", "Ignore events on this window"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bFilterZeroLegacyInputs ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bFilterZeroLegacyInputs = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("FilterZeroLegacyLabel", "Filters Legacy Inputs"))
				]
			]

			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8.f, 0.f, 0.f, 0.f)
			[
				SNew(SCheckBox)
				.IsChecked_Lambda([this]() { return bShowSlateMouseFocusEvents ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
				.OnCheckStateChanged_Lambda([this](ECheckBoxState State)
				{
					bShowSlateMouseFocusEvents = (State == ECheckBoxState::Checked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("FilterMouseFocusLabel", "Show Mouse Focus Events"))
				]
			]
		];

#if !UE_PLAYER_INPUT_INCLUDE_DEBUG
	VBox->AddSlot()
	.AutoHeight()
	.Padding(8.f, 4.f)
	[
		SNew(STextBlock)
		.Text(LOCTEXT("PlayerEventsNotAvailable",
			"Player input events are not available in this configuration. "
			"(Requires UE_PLAYER_INPUT_INCLUDE_DEBUG=1)"))
		.ColorAndOpacity(FLinearColor(0.9f, 0.6f, 0.2f))
		.AutoWrapText(true)
	];
#endif

	VBox->AddSlot()
	.FillHeight(1.f)
	[
		SAssignNew(ListView, SListView<TSharedPtr<FUnifiedInputEventRecord>>)
		.ListItemsSource(&LogItems)
		.OnGenerateRow(this, &SPlayerInputEventsTab::OnGenerateRow)
		.HeaderRow(HeaderRow)
		.SelectionMode(ESelectionMode::Single)
		.OnSelectionChanged_Lambda([this](TSharedPtr<FUnifiedInputEventRecord> Item, ESelectInfo::Type)
		{
			SelectedItem = Item;
			RebuildDetails();
		})
	];

	VBox->AddSlot()
	.AutoHeight()
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SBox)
			.MaxDesiredHeight(200.f)
			[
				SAssignNew(DetailsBox, SBox)
			]
		]
	];

	ChildSlot[ VBox ];

	RebuildDetails();

	// Store a weak reference to ourselves so we can identify our own window.
	WeakSelfWidget = SharedThis(this);

	BeginPIEHandle = FEditorDelegates::BeginPIE.AddRaw(this, &SPlayerInputEventsTab::OnBeginPIE);

#if WITH_SLATE_DEBUGGING
	SlateInputEventHandle    = FSlateDebugging::InputEvent.AddRaw(this, &SPlayerInputEventsTab::OnSlateInputEvent);
	SlateMouseCaptureHandle  = FSlateDebugging::MouseCaptureEvent.AddRaw(this, &SPlayerInputEventsTab::OnSlateMouseCaptureEvent);
#endif

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
	EventExecutedHandle = UE::Input::FPlayerInputDebugging::OnPlayerInputEventExecuted.AddRaw(
		this, &SPlayerInputEventsTab::OnPlayerInputEventExecuted);
	FlushedHandle = UE::Input::FPlayerInputDebugging::OnPlayerInputFlushed.AddRaw(
		this, &SPlayerInputEventsTab::OnPlayerInputFlushed);
#endif
}

SPlayerInputEventsTab::~SPlayerInputEventsTab()
{
	FEditorDelegates::BeginPIE.Remove(BeginPIEHandle);

#if WITH_SLATE_DEBUGGING
	FSlateDebugging::InputEvent.Remove(SlateInputEventHandle);
	FSlateDebugging::MouseCaptureEvent.Remove(SlateMouseCaptureHandle);
#endif

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
	UE::Input::FPlayerInputDebugging::OnPlayerInputEventExecuted.Remove(EventExecutedHandle);
	UE::Input::FPlayerInputDebugging::OnPlayerInputFlushed.Remove(FlushedHandle);
#endif
}

void SPlayerInputEventsTab::SetPlayerController(APlayerController* /*PC*/)
{
	// Player controller selection no longer filters events; all PCs are shown.
}

void SPlayerInputEventsTab::OnBeginPIE(const bool /*bIsSimulating*/)
{
	LogItems.Empty();
	SelectedItem.Reset();
	RebuildDetails();
	ListView->RequestListRefresh();
}

// ── AppendRecord ──────────────────────────────────────────────────────────────

void SPlayerInputEventsTab::AppendRecord(TSharedPtr<FUnifiedInputEventRecord> Record)
{
	if (LogItems.Num() >= MaxLogEntries)
	{
		LogItems.RemoveAt(0, EAllowShrinking::No);
	}
	LogItems.Add(MoveTemp(Record));
	ListView->RequestListRefresh();

	if (bAutoScroll && !SelectedItem.IsValid() && !LogItems.IsEmpty())
	{
		ListView->RequestScrollIntoView(LogItems.Last());
	}
}

// ── Row generation ────────────────────────────────────────────────────────────

TSharedRef<ITableRow> SPlayerInputEventsTab::OnGenerateRow(TSharedPtr<FUnifiedInputEventRecord> Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SUnifiedInputEventRow, OwnerTable)
		.Item(Item);
}

// ── Slate input event handler ─────────────────────────────────────────────────

#if WITH_SLATE_DEBUGGING

void SPlayerInputEventsTab::OnSlateInputEvent(const FSlateDebuggingInputEventArgs& EventArgs)
{
	// Always skip high-frequency mouse move events.
	if (EventArgs.InputEventType == ESlateDebuggingInputEvent::MouseMove)
	{
		return;
	}

	// Optionally filter mouse enter/leave (focus) events.
	if (!bShowSlateMouseFocusEvents &&
		(EventArgs.InputEventType == ESlateDebuggingInputEvent::MouseEnter ||
		 EventArgs.InputEventType == ESlateDebuggingInputEvent::MouseLeave))
	{
		return;
	}

	// Filter events that originate inside our own debugger window.
	if (bIgnoreDebuggerWindowEvents && EventArgs.HandlerWidget.IsValid())
	{
		if (TSharedPtr<SWidget> Self = WeakSelfWidget.Pin())
		{
			if (FSlateApplication::IsInitialized())
			{
				TSharedPtr<SWindow> HandlerWindow = FSlateApplication::Get().FindWidgetWindow(EventArgs.HandlerWidget.ToSharedRef());
				TSharedPtr<SWindow> SelfWindow    = FSlateApplication::Get().FindWidgetWindow(Self.ToSharedRef());
				if (HandlerWindow.IsValid() && HandlerWindow == SelfWindow)
				{
					return;
				}
			}
		}
	}

	TSharedPtr<FUnifiedInputEventRecord> Record = MakeShared<FUnifiedInputEventRecord>();
	Record->FrameNumber    = GFrameNumber;
	Record->Source         = EInputEventSource::Slate;
	Record->SlateEventType = EventArgs.InputEventType;
	Record->bReplyHandled  = EventArgs.Reply.IsEventHandled();
	Record->AdditionalContent = EventArgs.AdditionalContent;

	// Extract key and analog value from the typed FInputEvent pointer.
	if (const FInputEvent* IE = EventArgs.InputEvent)
	{
		switch (EventArgs.InputEventType)
		{
		case ESlateDebuggingInputEvent::PreviewKeyDown:
		case ESlateDebuggingInputEvent::KeyDown:
		case ESlateDebuggingInputEvent::KeyUp:
			{
				const FKeyEvent* KE = static_cast<const FKeyEvent*>(IE);
				Record->Key      = KE->GetKey();
				Record->RawValue = (EventArgs.InputEventType == ESlateDebuggingInputEvent::KeyUp) ? 0.f : 1.f;
			}
			break;

		case ESlateDebuggingInputEvent::AnalogInput:
			{
				const FAnalogInputEvent* AE = static_cast<const FAnalogInputEvent*>(IE);
				Record->Key      = AE->GetKey();
				Record->RawValue = AE->GetAnalogValue();
			}
			break;

		case ESlateDebuggingInputEvent::PreviewMouseButtonDown:
		case ESlateDebuggingInputEvent::MouseButtonDown:
		case ESlateDebuggingInputEvent::MouseButtonUp:
		case ESlateDebuggingInputEvent::MouseButtonDoubleClick:
			{
				const FPointerEvent* PE = static_cast<const FPointerEvent*>(IE);
				Record->Key      = PE->GetEffectingButton();
				Record->RawValue = (EventArgs.InputEventType == ESlateDebuggingInputEvent::MouseButtonUp) ? 0.f : 1.f;
			}
			break;

		case ESlateDebuggingInputEvent::MouseWheel:
			{
				const FPointerEvent* PE = static_cast<const FPointerEvent*>(IE);
				Record->RawValue = PE->GetWheelDelta();
			}
			break;

		default:
			break;
		}
	}

	// Handler widget info: type name + walk up to find the nearest UUserWidget.
	if (EventArgs.HandlerWidget.IsValid())
	{
		Record->HandlerWidgetType = EventArgs.HandlerWidget->GetType().ToString();

		TSharedPtr<SWidget> Current = EventArgs.HandlerWidget;
		while (Current.IsValid())
		{
			if (Current->GetType() == TEXT("SObjectWidget"))
			{
				if (UUserWidget* UW = static_cast<SObjectWidget*>(Current.Get())->GetWidgetObject())
				{
					Record->UserWidgetName      = UW->GetClass()->GetName();
					Record->UserWidgetBlueprint = UW->GetClass()->ClassGeneratedBy;
				}
				break;
			}
			Current = Current->GetParentWidget();
		}
	}

	AppendRecord(MoveTemp(Record));
}

void SPlayerInputEventsTab::OnSlateMouseCaptureEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs)
{
	// Filter events originating in our own debugger window.
	if (bIgnoreDebuggerWindowEvents && EventArgs.CaptureWidget.IsValid())
	{
		if (TSharedPtr<SWidget> Self = WeakSelfWidget.Pin())
		{
			if (FSlateApplication::IsInitialized())
			{
				TSharedPtr<SWindow> CaptureWindow = FSlateApplication::Get().FindWidgetWindow(
					ConstCastSharedRef<SWidget>(EventArgs.CaptureWidget.ToSharedRef()));
				TSharedPtr<SWindow> SelfWindow = FSlateApplication::Get().FindWidgetWindow(Self.ToSharedRef());
				if (CaptureWindow.IsValid() && CaptureWindow == SelfWindow)
				{
					return;
				}
			}
		}
	}

	TSharedPtr<FUnifiedInputEventRecord> Record = MakeShared<FUnifiedInputEventRecord>();
	Record->FrameNumber   = GFrameNumber;
	Record->Source        = EInputEventSource::SlateMouseCapture;
	Record->bReplyHandled = EventArgs.Captured; // true = captured, false = lost

	if (EventArgs.CaptureWidget.IsValid())
	{
		Record->HandlerWidgetType = EventArgs.CaptureWidget->GetType().ToString();
	}

	AppendRecord(MoveTemp(Record));
}

#endif  // WITH_SLATE_DEBUGGING

// ── Player input event handlers ───────────────────────────────────────────────

#if UE_PLAYER_INPUT_INCLUDE_DEBUG

void SPlayerInputEventsTab::OnPlayerInputEventExecuted(const UPlayerInput* PlayerInput, const UE::Input::FPlayerInputDebuggingArgs& Args)
{
	// Resolve the action name to a UInputAction asset (only succeeds for Enhanced Input actions).
	UInputAction* Action = FindObject<UInputAction>(nullptr, *Args.ActionName);

	// When filtering is enabled, drop legacy (non-Enhanced-Input) events whose value is zero.
	// These represent un-pressed bindings that fire every tick and clutter the log.
	if (bFilterZeroLegacyInputs && !Action && Args.InputValue.IsNearlyZero())
	{
		return;
	}

	TSharedPtr<FUnifiedInputEventRecord> Record = MakeShared<FUnifiedInputEventRecord>();
	Record->FrameNumber = GFrameNumber;
	Record->Source      = EInputEventSource::Player;
	Record->InputValue  = Args.InputValue;

	// Identify the owning player controller so the PC column can be populated.
	if (const APlayerController* OwningPC = PlayerInput->GetTypedOuter<APlayerController>())
	{
		Record->PlayerControllerName = OwningPC->GetName();
	}

	if (Action)
	{
		Record->ActionAsset = Action;
		Record->ActionName  = Action->GetName();
	}
	else
	{
		Record->ActionName = Args.ActionName;
	}

	if (const UInputComponent* IC = Args.InputComponent.Get())
	{
		Record->InputComponentName     = IC->GetName();
		Record->InputComponentPriority = IC->Priority;
		if (const AActor* Owner = IC->GetOwner())
		{
			Record->OwnerName      = Owner->GetActorNameOrLabel();
			Record->OwnerBlueprint = Owner->GetClass()->ClassGeneratedBy;
		}
		else if (const UObject* Outer = IC->GetOuter())
		{
			Record->OwnerName      = Outer->GetName();
			Record->OwnerBlueprint = Outer->GetClass()->ClassGeneratedBy;
		}
	}

	if (const UObject* Listener = Args.ListeningObject.Get())
	{
		if (const AActor* ListenerActor = Cast<AActor>(Listener))
		{
			Record->ListeningObjectName = ListenerActor->GetActorNameOrLabel();
		}
		else
		{
			Record->ListeningObjectName = Listener->GetName();
		}
		Record->ListeningObjectBlueprint = Listener->GetClass()->ClassGeneratedBy;
	}

	AppendRecord(MoveTemp(Record));
}

void SPlayerInputEventsTab::OnPlayerInputFlushed(const UPlayerInput* PlayerInput)
{
	TSharedPtr<FUnifiedInputEventRecord> Record = MakeShared<FUnifiedInputEventRecord>();
	Record->FrameNumber = GFrameNumber;
	Record->Source      = EInputEventSource::PlayerFlush;

	if (const APlayerController* OwningPC = PlayerInput->GetTypedOuter<APlayerController>())
	{
		Record->PlayerControllerName = OwningPC->GetName();
	}

	// Capture the BP VM callstack so we can show what triggered the flush.
	Record->ScriptCallstack = FFrame::GetScriptCallstack(/*bReturnEmpty*/ true);

	AppendRecord(MoveTemp(Record));
}

#endif  // UE_PLAYER_INPUT_INCLUDE_DEBUG

// ── Details panel ────────────────────────────────────────────────────────────

void SPlayerInputEventsTab::RebuildDetails()
{
	if (!SelectedItem.IsValid())
	{
		DetailsBox->SetContent(
			SNew(SBox).Padding(FMargin(8.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NoSelection", "Select a row to view details."))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
			]
		);
		return;
	}

	if (SelectedItem->Source == EInputEventSource::PlayerFlush)
	{
		TSharedRef<SVerticalBox> FlushBox = SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(8.f, 8.f, 8.f, 4.f))
			[
				SNew(STextBlock)
				.Text(FText::Format(
					LOCTEXT("FlushDetails", "Input Flush — Frame {0}  ({1})"),
					FText::AsNumber(SelectedItem->FrameNumber),
					FText::FromString(SelectedItem->PlayerControllerName)))
				.ColorAndOpacity(FLinearColor(0.9f, 0.2f, 0.2f))
			];

		if (!SelectedItem->ScriptCallstack.IsEmpty())
		{
			FlushBox->AddSlot()
			.AutoHeight()
			.Padding(FMargin(8.f, 4.f, 8.f, 2.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FlushScriptCallstackLabel", "Script Callstack:"))
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
			];

			FlushBox->AddSlot()
			.FillHeight(1.f)
			.Padding(FMargin(8.f, 0.f, 8.f, 8.f))
			[
				SNew(SMultiLineEditableTextBox)
				.IsReadOnly(true)
				.AllowMultiLine(true)
				.AlwaysShowScrollbars(true)
				.Text(FText::FromString(SelectedItem->ScriptCallstack))
			];
		}
		else
		{
			FlushBox->AddSlot()
			.AutoHeight()
			.Padding(FMargin(8.f, 4.f, 8.f, 8.f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("FlushNoScriptCallstack", "(Flush was not invoked from script.)"))
				.ColorAndOpacity(FLinearColor(0.5f, 0.5f, 0.5f))
			];
		}

		DetailsBox->SetContent(FlushBox);
		return;
	}

	TSharedPtr<FUnifiedInputEventRecord> Item = SelectedItem;

	// Returns a hyperlink if the object has a blueprint asset, otherwise plain text.
	auto MakeObjectWidget = [](const FString& Name, TWeakObjectPtr<UObject> Blueprint) -> TSharedRef<SWidget>
	{
		if (Blueprint.IsValid())
		{
			return SNew(SHyperlink)
				.Text(FText::FromString(Name))
				.OnNavigate(FSimpleDelegate::CreateLambda([Blueprint]()
				{
					if (UObject* BP = Blueprint.Get())
					{
						if (UAssetEditorSubsystem* Sub = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>())
						{
							Sub->OpenEditorForAsset(BP);
						}
					}
				}));
		}
		return SNew(STextBlock).Text(FText::FromString(Name));
	};

	auto MakeRow = [](const FText& Label, TSharedRef<SWidget> Value) -> TSharedRef<SWidget>
	{
		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(Label)
				.ColorAndOpacity(FLinearColor(0.6f, 0.6f, 0.6f))
				.MinDesiredWidth(170.f)
			]
			+ SHorizontalBox::Slot().AutoWidth().VAlign(VAlign_Center)
			[
				Value
			];
	};

	TSharedRef<SScrollBox> ScrollBox = SNew(SScrollBox);
	auto AddRow = [&](const FText& Label, TSharedRef<SWidget> Value)
	{
		ScrollBox->AddSlot().Padding(FMargin(8.f, 3.f))[ MakeRow(Label, Value) ];
	};

	AddRow(LOCTEXT("DetailFrame", "Frame:"),
		SNew(STextBlock).Text(FText::AsNumber(Item->FrameNumber)));

	if (Item->Source == EInputEventSource::Slate)
	{
		AddRow(LOCTEXT("DetailSource",     "Source:"),
			SNew(STextBlock).Text(LOCTEXT("SourceSlate", "Slate"))
				.ColorAndOpacity(FLinearColor(0.4f, 0.7f, 1.0f)));
		AddRow(LOCTEXT("DetailEventType",  "Event Type:"),
			SNew(STextBlock).Text(GetSlateEventTypeText(Item->SlateEventType)));
		AddRow(LOCTEXT("DetailKey",        "Key:"),
			SNew(STextBlock).Text(Item->Key.IsValid() ? FText::FromString(Item->Key.ToString()) : FText::GetEmpty()));
		AddRow(LOCTEXT("DetailValue",      "Value:"),
			SNew(STextBlock).Text(FText::FromString(FString::SanitizeFloat(Item->RawValue))));
		AddRow(LOCTEXT("DetailHandler",    "Handler Widget:"),
			MakeObjectWidget(Item->UserWidgetName.IsEmpty()
				? Item->HandlerWidgetType
				: FString::Printf(TEXT("%s (%s)"), *Item->HandlerWidgetType, *Item->UserWidgetName),
				Item->UserWidgetBlueprint));
		AddRow(LOCTEXT("DetailReply",      "Reply Handled:"),
			SNew(STextBlock).Text(Item->bReplyHandled ? LOCTEXT("Yes","Yes") : LOCTEXT("No","No"))
				.ColorAndOpacity(Item->bReplyHandled ? FLinearColor(0.2f,0.9f,0.2f) : FLinearColor(0.5f,0.5f,0.5f)));
		if (!Item->AdditionalContent.IsEmpty())
		{
			AddRow(LOCTEXT("DetailAdditional", "Additional:"),
				SNew(STextBlock).Text(FText::FromString(Item->AdditionalContent)));
		}
	}
	else if (Item->Source == EInputEventSource::SlateMouseCapture)
	{
		const bool bCaptured = Item->bReplyHandled;
		AddRow(LOCTEXT("DetailSource",        "Source:"),
			SNew(STextBlock).Text(LOCTEXT("SourceSlate", "Slate"))
				.ColorAndOpacity(FLinearColor(0.4f, 0.7f, 1.0f)));
		AddRow(LOCTEXT("DetailMouseCapEvent", "Event:"),
			SNew(STextBlock)
				.Text(bCaptured ? LOCTEXT("MouseCaptured", "Mouse Captured") : LOCTEXT("MouseCaptureLost", "Mouse Capture Lost"))
				.ColorAndOpacity(bCaptured ? FLinearColor(0.2f, 0.9f, 0.2f) : FLinearColor(0.9f, 0.6f, 0.2f)));
		AddRow(LOCTEXT("DetailCapWidget",     "Capturing Widget:"),
			SNew(STextBlock).Text(FText::FromString(Item->HandlerWidgetType)));
	}
	else
	{
		AddRow(LOCTEXT("DetailSource",     "Source:"),
			SNew(STextBlock).Text(LOCTEXT("SourcePlayer", "Player"))
				.ColorAndOpacity(FLinearColor(0.6f, 0.9f, 0.6f)));
		AddRow(LOCTEXT("DetailPC",         "Player Controller:"),
			SNew(STextBlock).Text(FText::FromString(Item->PlayerControllerName)));
		AddRow(LOCTEXT("DetailComponent",  "Input Component:"),
			SNew(STextBlock).Text(FText::FromString(Item->InputComponentName)));
		AddRow(LOCTEXT("DetailPriority",   "Priority:"),
			SNew(STextBlock).Text(FText::AsNumber(Item->InputComponentPriority)));
		AddRow(LOCTEXT("DetailOwner",      "Owner:"),
			MakeObjectWidget(Item->OwnerName, Item->OwnerBlueprint));
		AddRow(LOCTEXT("DetailListener",   "Listening Object:"),
			MakeObjectWidget(Item->ListeningObjectName, Item->ListeningObjectBlueprint));
		AddRow(LOCTEXT("DetailAction",     "Action:"),
			MakeObjectWidget(Item->ActionName, Item->ActionAsset));
		AddRow(LOCTEXT("DetailValue",      "Value:"),
			SNew(STextBlock).Text(FText::FromString(FormatInputValue(Item->InputValue))));
	}

	DetailsBox->SetContent(ScrollBox);
}

#undef LOCTEXT_NAMESPACE
