// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Debugging/SlateDebugging.h"
#include "GameFramework/PlayerController.h"
#include "GameFramework/PlayerInput.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/ITableRow.h"
#include "Widgets/Views/SListView.h"
#include "GameFramework/PlayerInputDebugging.h"

// Discriminates each log entry in the combined Input Events tab.
enum class EInputEventSource : uint8
{
	Slate,             // from FSlateDebugging::InputEvent
	SlateMouseCapture, // from FSlateDebugging::MouseCaptureEvent
	Player,            // from FPlayerInputDebugging::OnPlayerInputEventExecuted
	PlayerFlush,       // flush marker from FPlayerInputDebugging::OnPlayerInputFlushed
};

// A single entry in the combined input event log.
struct FUnifiedInputEventRecord
{
	uint32 FrameNumber = 0;
	EInputEventSource Source = EInputEventSource::Slate;

	// ── Slate fields ─────────────────────────────────────────────────────────
	ESlateDebuggingInputEvent SlateEventType = ESlateDebuggingInputEvent::MAX;
	FKey Key;
	float RawValue = 0.f;
	FString HandlerWidgetType;
	FString UserWidgetName;                      // nearest UUserWidget class name, if any
	TWeakObjectPtr<UObject> UserWidgetBlueprint; // ClassGeneratedBy for hyperlink
	FString AdditionalContent;
	bool bReplyHandled = false; // Slate: reply handled; SlateMouseCapture: true=captured, false=lost

	// ── Player fields ─────────────────────────────────────────────────────────
	FString PlayerControllerName; // PC that owns the PlayerInput; empty for Slate events
	FString ActionName;
	TWeakObjectPtr<UObject> ActionAsset;
	FString InputComponentName;
	int32 InputComponentPriority = 0;
	FString OwnerName;
	TWeakObjectPtr<UObject> OwnerBlueprint;
	FString ListeningObjectName;
	TWeakObjectPtr<UObject> ListeningObjectBlueprint;
	FVector InputValue = FVector::ZeroVector;

	// ── Flush fields ──────────────────────────────────────────────────────────
	// Script (Blueprint VM) callstack captured at the moment the flush happened,
	// so the user can see what actually triggered it. Empty if the flush wasn't
	// invoked from script.
	FString ScriptCallstack;
};

// Combined Input Events tab: Slate input events (FSlateDebugging::InputEvent) and
// Player input events (FPlayerInputDebugging) in a single clickable log.
class SPlayerInputEventsTab : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlayerInputEventsTab) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	virtual ~SPlayerInputEventsTab();

	void SetPlayerController(APlayerController* PC);

private:
	TSharedRef<ITableRow> OnGenerateRow(TSharedPtr<FUnifiedInputEventRecord> Item, const TSharedRef<STableViewBase>& OwnerTable);
	void AppendRecord(TSharedPtr<FUnifiedInputEventRecord> Record);
	void RebuildDetails();
	void OnBeginPIE(const bool bIsSimulating);

#if WITH_SLATE_DEBUGGING
	void OnSlateInputEvent(const FSlateDebuggingInputEventArgs& EventArgs);
	void OnSlateMouseCaptureEvent(const FSlateDebuggingMouseCaptureEventArgs& EventArgs);
	FDelegateHandle SlateInputEventHandle;
	FDelegateHandle SlateMouseCaptureHandle;
#endif

#if UE_PLAYER_INPUT_INCLUDE_DEBUG
	void OnPlayerInputEventExecuted(const UPlayerInput* PlayerInput, const UE::Input::FPlayerInputDebuggingArgs& Args);
	void OnPlayerInputFlushed(const UPlayerInput* PlayerInput);
	FDelegateHandle EventExecutedHandle;
	FDelegateHandle FlushedHandle;
#endif

	FDelegateHandle BeginPIEHandle;
	TWeakPtr<SWidget> WeakSelfWidget; // used to identify our window for event filtering

	TArray<TSharedPtr<FUnifiedInputEventRecord>> LogItems;
	TSharedPtr<SListView<TSharedPtr<FUnifiedInputEventRecord>>> ListView;
	bool bAutoScroll = true;
	bool bIgnoreDebuggerWindowEvents = true;
	bool bFilterZeroLegacyInputs = true;
	bool bShowSlateMouseFocusEvents = false;

	TSharedPtr<FUnifiedInputEventRecord> SelectedItem;
	TSharedPtr<SBox> DetailsBox;

	static constexpr int32 MaxLogEntries = 1024;
};
